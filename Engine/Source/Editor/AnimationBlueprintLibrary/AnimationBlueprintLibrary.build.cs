// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class AnimationBlueprintLibrary : ModuleRules
{
	public AnimationBlueprintLibrary(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"TimeManagement"
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AnimGraph",
				"Engine",
				"BlueprintGraph",
				"UnrealEd",
			}
		);
	}
}
