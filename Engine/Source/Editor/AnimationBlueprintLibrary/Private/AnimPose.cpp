// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimPose.h"

#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimNode_LinkedInputPose.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AttributesRuntime.h"
#include "Animation/Skeleton.h"
#include "Animation/SmartName.h"
#include "AnimationBlueprintLibrary.h"
#include "AnimationRuntime.h"
#include "BoneContainer.h"
#include "BoneIndices.h"
#include "BonePose.h"
#include "Components/SkeletalMeshComponent.h"
#include "Containers/BitArray.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Engine/SkeletalMesh.h"
#include "HAL/PlatformCrt.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/TransformVectorized.h"
#include "Misc/AssertionMacros.h"
#include "Misc/MemStack.h"
#include "PreviewScene.h"
#include "ReferenceSkeleton.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Templates/Casts.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimPose)

DEFINE_LOG_CATEGORY_STATIC(LogAnimationPoseScripting, Verbose, All);

void FAnimPose::Init(const FBoneContainer& InBoneContainer)
{
	Reset();

	const FReferenceSkeleton& RefSkeleton = InBoneContainer.GetSkeletonAsset()->GetReferenceSkeleton();
	
	for (const FBoneIndexType BoneIndex : InBoneContainer.GetBoneIndicesArray())
	{			
		const FCompactPoseBoneIndex CompactIndex(BoneIndex);
		const FCompactPoseBoneIndex CompactParentIndex = InBoneContainer.GetParentBoneIndex(CompactIndex);

		const int32 SkeletonBoneIndex = InBoneContainer.GetSkeletonIndex(CompactIndex);
		if (SkeletonBoneIndex != INDEX_NONE)
		{
			const int32 ParentBoneIndex = CompactParentIndex.GetInt() != INDEX_NONE ? InBoneContainer.GetSkeletonIndex(CompactParentIndex) : INDEX_NONE;

			BoneIndices.Add(SkeletonBoneIndex);
			ParentBoneIndices.Add(ParentBoneIndex);

			BoneNames.Add(RefSkeleton.GetBoneName(SkeletonBoneIndex));

			RefLocalSpacePoses.Add(InBoneContainer.GetRefPoseTransform(FCompactPoseBoneIndex(BoneIndex)));
		}
	}

	TArray<bool> Processed;
	Processed.SetNumZeroed(BoneNames.Num());
	RefWorldSpacePoses.SetNum(BoneNames.Num());
	for (int32 EntryIndex = 0; EntryIndex < BoneNames.Num(); ++EntryIndex)
	{
		const int32 ParentIndex = ParentBoneIndices[EntryIndex];
		const int32 TransformIndex = BoneIndices.IndexOfByKey(ParentIndex);

		if (TransformIndex != INDEX_NONE)
		{
			ensure(Processed[TransformIndex]);
			RefWorldSpacePoses[EntryIndex] = RefLocalSpacePoses[EntryIndex] * RefWorldSpacePoses[TransformIndex];
		}
		else
		{
			RefWorldSpacePoses[EntryIndex] = RefLocalSpacePoses[EntryIndex];
		}

		Processed[EntryIndex] = true;
	}
}

void FAnimPose::GetPose(FCompactPose& InOutCompactPose) const
{
	if (IsValid())
	{
		for (int32 Index = 0; Index < BoneNames.Num(); ++Index)
		{
			const FName& BoneName = BoneNames[Index];
			const int32 MeshBoneIndex = InOutCompactPose.GetBoneContainer().GetPoseBoneIndexForBoneName(BoneName);
			const FCompactPoseBoneIndex PoseBoneIndex = InOutCompactPose.GetBoneContainer().MakeCompactPoseIndex(FMeshPoseBoneIndex(MeshBoneIndex));
			if (PoseBoneIndex != INDEX_NONE)
			{
				InOutCompactPose[PoseBoneIndex] = LocalSpacePoses[Index];
			}
		}
	}
}

void FAnimPose::SetPose(USkeletalMeshComponent* Component)
{
	if (IsInitialized())
	{		
		const FBoneContainer& ContextBoneContainer = Component->GetAnimInstance()->GetRequiredBones();

		LocalSpacePoses.SetNum(RefLocalSpacePoses.Num());

		TArray<FTransform> BoneSpaceTransforms = Component->GetBoneSpaceTransforms();
		for (const FBoneIndexType BoneIndex : ContextBoneContainer.GetBoneIndicesArray())
		{
			const int32 SkeletonBoneIndex = ContextBoneContainer.GetSkeletonIndex(FCompactPoseBoneIndex(BoneIndex));
			if (SkeletonBoneIndex != INDEX_NONE)
			{
				const int32 LocalBoneIndex = BoneIndices.IndexOfByKey(SkeletonBoneIndex);
				if (LocalBoneIndex != INDEX_NONE)
				{
					LocalSpacePoses[LocalBoneIndex] = BoneSpaceTransforms[BoneIndex];
				}
				else
				{
					UE_LOGF(LogAnimationPoseScripting, Warning, "Failed to find local Bone Index for Skeleton Bone Index %i [%ls]", SkeletonBoneIndex, *ContextBoneContainer.GetReferenceSkeleton().GetBoneName(SkeletonBoneIndex).ToString());
				}
			}
			else
			{
				UE_LOGF(LogAnimationPoseScripting, Warning, "Failed to find Skeleton index for Compact Pose Bone Index %i", BoneIndex);
			}
		}

		GenerateWorldSpaceTransforms();
	}
}

void FAnimPose::GenerateWorldSpaceTransforms()
{
	if (IsPopulated())
	{
		TArray<bool> Processed;
		Processed.SetNumZeroed(BoneNames.Num());
		WorldSpacePoses.SetNum(BoneNames.Num());
		for (int32 EntryIndex = 0; EntryIndex < BoneNames.Num(); ++EntryIndex)
		{
			const int32 ParentIndex = ParentBoneIndices[EntryIndex];
			const int32 TransformIndex = BoneIndices.IndexOfByKey(ParentIndex);
			if (TransformIndex != INDEX_NONE)
			{
				ensure(Processed[TransformIndex]);
				WorldSpacePoses[EntryIndex] = LocalSpacePoses[EntryIndex] * WorldSpacePoses[TransformIndex];
			}
			else
			{
				WorldSpacePoses[EntryIndex] = LocalSpacePoses[EntryIndex];
			}

			Processed[EntryIndex] = true;
		}
	}
	else
	{
		UE_LOGF(LogAnimationPoseScripting, Warning, "Anim Pose was not previously populated");
	}
}

void FAnimPose::SetPose(const FAnimationPoseData& PoseData)
{
	const FCompactPose& CompactPose = PoseData.GetPose();
	if (IsInitialized())
	{
		const FBoneContainer& ContextBoneContainer = CompactPose.GetBoneContainer();
			
		LocalSpacePoses.SetNum(RefLocalSpacePoses.Num());
		for (const FCompactPoseBoneIndex BoneIndex : CompactPose.ForEachBoneIndex())
		{
			const int32 SkeletonBoneIndex = ContextBoneContainer.GetSkeletonIndex(BoneIndex);
			if (SkeletonBoneIndex != INDEX_NONE)
			{
				const int32 LocalBoneIndex = BoneIndices.IndexOfByKey(SkeletonBoneIndex);
				if (LocalBoneIndex != INDEX_NONE)
				{
					LocalSpacePoses[LocalBoneIndex] = CompactPose[BoneIndex];
				}
				else
				{
					UE_LOGF(LogAnimationPoseScripting, Warning, "Failed to find local Bone Index for Skeleton Bone Index %i [%ls]", SkeletonBoneIndex, *ContextBoneContainer.GetReferenceSkeleton().GetBoneName(SkeletonBoneIndex).ToString());
				}
			}
			else
			{
				UE_LOGF(LogAnimationPoseScripting, Warning, "Failed to find Skeleton index for Compact Pose Bone Index %i", BoneIndex.GetInt());
			}
		}

		ensure(LocalSpacePoses.Num() == RefLocalSpacePoses.Num());
		GenerateWorldSpaceTransforms();

		const FBlendedCurve& Curve = PoseData.GetCurve();
		Curve.ForEachElement([&CurveNames = CurveNames, &CurveValues = CurveValues](const UE::Anim::FCurveElement& InElement)
		{
			CurveNames.Add(InElement.Name);
			CurveValues.Add(InElement.Value);
		});

		TArray<USkeletalMeshSocket*> Sockets;
		const USkeleton* Skeleton = ContextBoneContainer.GetSkeletonAsset();
		const USkeletalMesh* SkeletalMesh = ContextBoneContainer.GetSkeletalMeshAsset();
		if (SkeletalMesh)
		{
			Sockets = SkeletalMesh->GetActiveSocketList();
		}
		else if (Skeleton)
		{
			Sockets = Skeleton->Sockets;
		}

		for (const USkeletalMeshSocket* Socket : Sockets)
		{
			const int32 PoseBoneIndex = ContextBoneContainer.GetPoseBoneIndexForBoneName(Socket->BoneName);
			if (PoseBoneIndex != INDEX_NONE)
			{
				SocketNames.Add(Socket->SocketName);
				SocketParentBoneNames.Add(Socket->BoneName);
				SocketTransforms.Add(Socket->GetSocketLocalTransform());
			}
		}		
	}
	else
	{
		UE_LOGF(LogAnimationPoseScripting, Warning, "Anim Pose was not previously initialized");
	}
}

void FAnimPose::SetToRefPose()
{
	if (IsInitialized())
	{
		LocalSpacePoses = RefLocalSpacePoses;
		WorldSpacePoses = RefWorldSpacePoses;
	}
	else
	{
		UE_LOGF(LogAnimationPoseScripting, Warning, "Anim Pose was not previously initialized");
	}
}

bool FAnimPose::IsValid() const
{
	const int32 ExpectedNumBones = BoneNames.Num();
	bool bIsValid = ExpectedNumBones != 0;
	
	bIsValid &= BoneIndices.Num() == ExpectedNumBones;
	bIsValid &= ParentBoneIndices.Num() == ExpectedNumBones;
	bIsValid &= LocalSpacePoses.Num() == ExpectedNumBones;
	bIsValid &= WorldSpacePoses.Num() == ExpectedNumBones;
	bIsValid &= RefLocalSpacePoses.Num() == ExpectedNumBones;
	bIsValid &= RefWorldSpacePoses.Num() == ExpectedNumBones;
	
	return bIsValid;
}

void FAnimPose::Reset()
{
	BoneNames.Empty();
	BoneIndices.Empty();
	ParentBoneIndices.Empty();
	LocalSpacePoses.Empty();
	WorldSpacePoses.Empty();
	RefLocalSpacePoses.Empty();
	RefWorldSpacePoses.Empty();
}

bool UAnimPoseExtensions::IsValid(const FAnimPose& Pose)
{
	return Pose.IsValid();
}

void UAnimPoseExtensions::GetBoneNames(const FAnimPose& Pose, TArray<FName>& Bones)
{
	Bones.Append(Pose.BoneNames);
}

const FTransform& UAnimPoseExtensions::GetBonePose(const FAnimPose& Pose, FName BoneName, EAnimPoseSpaces Space /*= EAnimPoseSpaces::Local*/)
{
	if (Pose.IsValid())
	{
		const int32 BoneIndex = Pose.BoneNames.IndexOfByKey(BoneName);
		if (BoneIndex != INDEX_NONE)
		{		
			return Space == EAnimPoseSpaces::Local ? Pose.LocalSpacePoses[BoneIndex] : Pose.WorldSpacePoses[BoneIndex];		
		}
		else
		{
			UE_LOGF(LogAnimationPoseScripting, Warning, "No bone with name %ls was found", *BoneName.ToString());
		}
	}
	else
	{		
		UE_LOGF(LogAnimationPoseScripting, Error, "Provided Pose is not valid");	
	}
	return FTransform::Identity;
}

void UAnimPoseExtensions::SetBonePose(FAnimPose& Pose, FTransform Transform, FName BoneName, EAnimPoseSpaces Space /*= EAnimPoseSpaces::Local*/)
{
	if (Pose.IsValid())
	{
		const int32 BoneIndex = Pose.BoneNames.IndexOfByKey(BoneName);
		if (BoneIndex != INDEX_NONE)
		{
			if (Space == EAnimPoseSpaces::Local)
			{
				Pose.LocalSpacePoses[BoneIndex] = Transform;
			}
			else if (Space == EAnimPoseSpaces::World)
			{
				const int32 ParentIndex = Pose.ParentBoneIndices[BoneIndex];
				const FTransform ParentTransformWS = ParentIndex != INDEX_NONE ? Pose.WorldSpacePoses[ParentIndex] : FTransform::Identity;
				Pose.LocalSpacePoses[BoneIndex] = Transform.GetRelativeTransform(ParentTransformWS);
			}
			
			Pose.GenerateWorldSpaceTransforms();
		}
		else
		{
			UE_LOGF(LogAnimationPoseScripting, Warning, "No bone with name %ls was found", *BoneName.ToString());
		}
	}
	else
	{		
		UE_LOGF(LogAnimationPoseScripting, Error, "Provided Pose is not valid");	
	}
}
	
const FTransform& UAnimPoseExtensions::GetRefBonePose(const FAnimPose& Pose, FName BoneName, EAnimPoseSpaces Space /*= EAnimPoseSpaces::Local*/)
{
	if (Pose.IsValid())
	{
		const int32 BoneIndex = Pose.BoneNames.IndexOfByKey(BoneName);
		if (BoneIndex != INDEX_NONE)
		{	
			return Space == EAnimPoseSpaces::Local ? Pose.RefLocalSpacePoses[BoneIndex] : Pose.RefWorldSpacePoses[BoneIndex];
		}
		else
		{
			UE_LOGF(LogAnimationPoseScripting, Warning, "No bone with name %ls was found", *BoneName.ToString());
		}
	}
	else
	{		
		UE_LOGF(LogAnimationPoseScripting, Error, "Provided Pose is not valid");	
	}

	return FTransform::Identity;	
}

FTransform UAnimPoseExtensions::GetRelativeTransform(const FAnimPose& Pose, FName FromBoneName, FName ToBoneName, EAnimPoseSpaces Space /*= EAnimPoseSpaces::Local*/)
{
	if (Pose.IsValid())
	{
		const int32 FromBoneIndex = Pose.BoneNames.IndexOfByKey(FromBoneName);
		const int32 ToBoneIndex = Pose.BoneNames.IndexOfByKey(ToBoneName);
		if (FromBoneIndex != INDEX_NONE && ToBoneIndex != INDEX_NONE)
		{	
			const FTransform& From = Space == EAnimPoseSpaces::Local ? Pose.LocalSpacePoses[FromBoneIndex] : Pose.WorldSpacePoses[FromBoneIndex];
			const FTransform& To = Space == EAnimPoseSpaces::Local ? Pose.LocalSpacePoses[ToBoneIndex] : Pose.WorldSpacePoses[ToBoneIndex];

			const FTransform Relative = To.GetRelativeTransform(From);
				
			return Relative;
		}
		else
		{
			UE_LOGF(LogAnimationPoseScripting, Warning, "No bone with name %ls or %ls was found", *FromBoneName.ToString(), *ToBoneName.ToString());
		}
	}
	else
	{		
		UE_LOGF(LogAnimationPoseScripting, Error, "Provided Pose is not valid");	
	}

	return FTransform::Identity;
}

FTransform UAnimPoseExtensions::GetRelativeToRefPoseTransform(const FAnimPose& Pose, FName BoneName, EAnimPoseSpaces Space /*= EAnimPoseSpaces::Local*/)
{
	if (Pose.IsValid())
	{
		const int32 BoneIndex = Pose.BoneNames.IndexOfByKey(BoneName);
		if (BoneIndex != INDEX_NONE)
		{	
			const FTransform& From = Space == EAnimPoseSpaces::Local ? Pose.RefLocalSpacePoses[BoneIndex] : Pose.RefWorldSpacePoses[BoneIndex];
			const FTransform& To = Space == EAnimPoseSpaces::Local ? Pose.LocalSpacePoses[BoneIndex] : Pose.WorldSpacePoses[BoneIndex];

			return To.GetRelativeTransform(From);
		}
		else
		{
			UE_LOGF(LogAnimationPoseScripting, Warning, "No bone with name %ls was found", *BoneName.ToString());
		}
	}
	else
	{		
		UE_LOGF(LogAnimationPoseScripting, Error, "Provided Pose is not valid");	
	}

	return FTransform::Identity;
}

FTransform UAnimPoseExtensions::GetRefPoseRelativeTransform(const FAnimPose& Pose, FName FromBoneName, FName ToBoneName, EAnimPoseSpaces Space /*= EAnimPoseSpaces::Local*/)
{
	if (Pose.IsValid())
	{
		const int32 FromBoneIndex = Pose.BoneNames.IndexOfByKey(FromBoneName);
		const int32 ToBoneIndex = Pose.BoneNames.IndexOfByKey(ToBoneName);
		if (FromBoneIndex != INDEX_NONE && ToBoneIndex != INDEX_NONE)
		{	
			const FTransform& From = Space == EAnimPoseSpaces::Local ? Pose.RefLocalSpacePoses[FromBoneIndex] : Pose.RefWorldSpacePoses[FromBoneIndex];
			const FTransform& To = Space == EAnimPoseSpaces::Local ? Pose.RefLocalSpacePoses[ToBoneIndex] : Pose.RefWorldSpacePoses[ToBoneIndex];

			const FTransform Relative = From.GetRelativeTransform(To);
				
			return Relative;
		}
		else
		{
			UE_LOGF(LogAnimationPoseScripting, Warning, "No bone with name %ls or %ls was found", *FromBoneName.ToString(), *ToBoneName.ToString());
		}
	}
	else
	{		
		UE_LOGF(LogAnimationPoseScripting, Error, "Provided Pose is not valid");	
	}

	return FTransform::Identity;
}

void UAnimPoseExtensions::GetSocketNames(const FAnimPose& Pose, TArray<FName>& Sockets)
{
	if (Pose.IsValid())
	{
		Sockets = Pose.SocketNames;
	}
}

FTransform UAnimPoseExtensions::GetSocketPose(const FAnimPose& Pose, FName SocketName, EAnimPoseSpaces Space)
{
	if (Pose.IsValid())
	{
		const int32 SocketIndex = Pose.SocketNames.IndexOfByKey(SocketName);			
		if (SocketIndex != INDEX_NONE)
		{
			const int32 BoneIndex = Pose.BoneNames.IndexOfByKey(Pose.SocketParentBoneNames[SocketIndex]);
			const FTransform BoneTransform = Space == EAnimPoseSpaces::Local ? Pose.LocalSpacePoses[BoneIndex] : Pose.WorldSpacePoses[BoneIndex];
			return Pose.SocketTransforms[SocketIndex] * BoneTransform;
		}
		else
		{
			UE_LOGF(LogAnimationPoseScripting, Warning, "No socket with name %ls was found", *SocketName.ToString());
		}
	}
	else
	{		
		UE_LOGF(LogAnimationPoseScripting, Error, "Provided Pose is not valid");	
	}
		
	return FTransform::Identity;
}

void UAnimPoseExtensions::EvaluateAnimationBlueprintWithInputPose(const FAnimPose& Pose, USkeletalMesh* TargetSkeletalMesh, UAnimBlueprint* AnimationBlueprint, FAnimPose& OutPose)
{
	if (Pose.IsValid())
	{
		if (TargetSkeletalMesh)
		{
			if (AnimationBlueprint)
			{
				UAnimBlueprintGeneratedClass* AnimGeneratedClass = AnimationBlueprint->GetAnimBlueprintGeneratedClass();
				if (AnimGeneratedClass)
				{
					if (AnimGeneratedClass->TargetSkeleton == TargetSkeletalMesh->GetSkeleton())
					{
						FMemMark Mark(FMemStack::Get());
						
						FPreviewScene PreviewScene;
			
						USkeletalMeshComponent* Component = NewObject<USkeletalMeshComponent>();
						Component->SetSkeletalMesh(TargetSkeletalMesh);
						Component->SetAnimInstanceClass(AnimationBlueprint->GetAnimBlueprintGeneratedClass());

						PreviewScene.AddComponent(Component, FTransform::Identity);
						Component->InitAnim(true);
			
						if (UAnimInstance* AnimInstance = Component->GetAnimInstance())
						{
							if (FAnimNode_LinkedInputPose* InputNode = AnimInstance->GetLinkedInputPoseNode())
							{
								const FBoneContainer& BoneContainer = AnimInstance->GetRequiredBones();
								InputNode->CachedInputPose.SetBoneContainer(&BoneContainer);
								InputNode->CachedInputCurve.InitFrom(BoneContainer);
								InputNode->CachedInputPose.ResetToRefPose();
								InputNode->bIsCachedInputPoseInitialized = true;

								// Copy bone transform from input pose using skeleton index mapping
								for (FCompactPoseBoneIndex CompactIndex : InputNode->CachedInputPose.ForEachBoneIndex())
								{
									const int32 SkeletonIndex = BoneContainer.GetSkeletonIndex(CompactIndex);
									if (SkeletonIndex != INDEX_NONE)
									{
										const int32 Index = Pose.BoneIndices.IndexOfByKey(SkeletonIndex);
										if (Index != INDEX_NONE)
										{
											InputNode->CachedInputPose[CompactIndex] = Pose.LocalSpacePoses[Index];
										}
									}
								}
					
								OutPose.Init(AnimInstance->GetRequiredBones());

								Component->TickAnimation(0.f, true);
								Component->RefreshBoneTransforms();
								const TArray<FTransform>& LocalSpaceTransforms = Component->GetBoneSpaceTransforms();

								OutPose.SetPose(Component);	
							}	
						}
						else
						{
							UE_LOGF(LogAnimationPoseScripting, Warning, "Failed to retrieve Input Pose Node from Animation Graph %ls", *AnimationBlueprint->GetName());	
						}
					}
					else
					{
						UE_LOGF(LogAnimationPoseScripting, Error, "Animation Blueprint target Skeleton %ls does not match Target Skeletal Mesh its Skeleton %ls", *AnimGeneratedClass->TargetSkeleton->GetName(), *TargetSkeletalMesh->GetSkeleton()->GetName());	
					}
				
				}
				else
				{
					UE_LOGF(LogAnimationPoseScripting, Warning, "Failed to retrieve Animation Blueprint generated class");	
				}
			}
			else
			{
				UE_LOGF(LogAnimationPoseScripting, Warning, "Invalid Animation Blueprint");			
			}	
		}
		else
		{		
			UE_LOGF(LogAnimationPoseScripting, Warning, "Invalid Target Skeletal Mesh");
		}
	}
	else
	{		
		UE_LOGF(LogAnimationPoseScripting, Error, "Provided Pose is not valid");	
	}
}

void UAnimPoseExtensions::GetReferencePose(USkeleton* Skeleton, FAnimPose& OutPose)
{
	if (Skeleton)
	{
		const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
				       
		TArray<FBoneIndexType> RequiredBoneIndexArray;
		RequiredBoneIndexArray.AddUninitialized(RefSkeleton.GetNum());
		for (int32 BoneIndex = 0; BoneIndex < RequiredBoneIndexArray.Num(); ++BoneIndex)
		{
			RequiredBoneIndexArray[BoneIndex] = IntCastChecked<FBoneIndexType>(BoneIndex);
		}

		FBoneContainer RequiredBones;
		RequiredBones.InitializeTo(RequiredBoneIndexArray, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *Skeleton);

		OutPose.Init(RequiredBones);
		OutPose.SetToRefPose();
	}
	else
	{		
		UE_LOGF(LogAnimationPoseScripting, Error, "Invalid Skeleton provided");	
	}
}

void UAnimPoseExtensions::GetCurveNames(const FAnimPose& Pose, TArray<FName>& Curves)
{
	Curves.Append(Pose.CurveNames);	
}

float UAnimPoseExtensions::GetCurveWeight(const FAnimPose& Pose, const FName& CurveName)
{
	float CurveValue = 0.f;	
	const int32 CurveIndex = Pose.CurveNames.IndexOfByKey(CurveName);
	if (CurveIndex != INDEX_NONE)
	{
		CurveValue = Pose.CurveValues[CurveIndex];
	}
	
	return CurveValue;
}

void UAnimPoseExtensions::GetAnimPoseAtFrame(const UAnimSequenceBase* AnimationSequenceBase, int32 FrameIndex, FAnimPoseEvaluationOptions EvaluationOptions, FAnimPose& Pose)
{
	if (AnimationSequenceBase)
	{
		const double Time = AnimationSequenceBase->GetDataModel()->GetFrameRate().AsSeconds(FrameIndex);
		GetAnimPoseAtTime(AnimationSequenceBase, Time, EvaluationOptions, Pose);
	}
}

void UAnimPoseExtensions::GetAnimPoseAtTime(const UAnimSequenceBase* AnimationSequenceBase, double Time, FAnimPoseEvaluationOptions EvaluationOptions, FAnimPose& Pose)
{
	TArray<FAnimPose> InOutPoses;
	GetAnimPoseAtTimeIntervals(AnimationSequenceBase, { Time }, EvaluationOptions, InOutPoses);

	if (InOutPoses.Num())
	{
		ensure(InOutPoses.Num() == 1);
		Pose = InOutPoses[0];
	}
} 

void UAnimPoseExtensions::GetAnimPoseAtTimeIntervals(const UAnimSequenceBase* AnimationSequenceBase, TArray<double> TimeIntervals, FAnimPoseEvaluationOptions EvaluationOptions, TArray<FAnimPose>& InOutPoses)
{
	if (AnimationSequenceBase && AnimationSequenceBase->GetSkeleton())
	{
		FMemMark Mark(FMemStack::Get());
		
		// asset to use for retarget proportions (can be either USkeletalMesh or USkeleton)
		UObject* AssetToUse;
		int32 NumRequiredBones;
		if (EvaluationOptions.OptionalSkeletalMesh)
		{
			AssetToUse = CastChecked<UObject>(EvaluationOptions.OptionalSkeletalMesh);
			NumRequiredBones = EvaluationOptions.OptionalSkeletalMesh->GetRefSkeleton().GetNum();	
		}
		else
		{
			AssetToUse = CastChecked<UObject>(AnimationSequenceBase->GetSkeleton());
			NumRequiredBones = AnimationSequenceBase->GetSkeleton()->GetReferenceSkeleton().GetNum();
		}

		TArray<FBoneIndexType> RequiredBoneIndexArray;
		RequiredBoneIndexArray.AddUninitialized(NumRequiredBones);
		for (int32 BoneIndex = 0; BoneIndex < RequiredBoneIndexArray.Num(); ++BoneIndex)
		{
			RequiredBoneIndexArray[BoneIndex] = static_cast<FBoneIndexType>(BoneIndex);
		}

		FBoneContainer RequiredBones;
		RequiredBones.InitializeTo(RequiredBoneIndexArray, UE::Anim::FCurveFilterSettings(EvaluationOptions.bEvaluateCurves ? UE::Anim::ECurveFilterMode::None : UE::Anim::ECurveFilterMode::DisallowAll), *AssetToUse);
		
		RequiredBones.SetUseRAWData(EvaluationOptions.EvaluationType == EAnimDataEvalType::Raw);
		RequiredBones.SetUseSourceData(EvaluationOptions.EvaluationType == EAnimDataEvalType::Source);
		
		RequiredBones.SetDisableRetargeting(!EvaluationOptions.bShouldRetarget);
		
		FCompactPose CompactPose;
        FBlendedCurve Curve;
        UE::Anim::FStackAttributeContainer Attributes;

        FAnimationPoseData PoseData(CompactPose, Curve, Attributes);
        FAnimExtractContext Context(0.0, EvaluationOptions.bExtractRootMotion);
#if WITH_EDITOR
		Context.bIgnoreRootLock = EvaluationOptions.bIncorporateRootMotionIntoPose;
#endif // WITH_EDITOR
    
        FCompactPose BasePose;
        BasePose.SetBoneContainer(&RequiredBones);
        
        CompactPose.SetBoneContainer(&RequiredBones);
        Curve.InitFrom(RequiredBones);

		FAnimPose Pose;
		Pose.Init(RequiredBones);
		
		for (int32 Index = 0; Index < TimeIntervals.Num(); ++Index)
		{
			const double EvalInterval = TimeIntervals[Index];
			
			bool bValidTime = false;
			UAnimationBlueprintLibrary::IsValidTime(AnimationSequenceBase, static_cast<float>(EvalInterval), bValidTime);
			ensure(bValidTime);

			Context.CurrentTime = EvalInterval;

			FAnimPose& FramePose = InOutPoses.AddDefaulted_GetRef();
			FramePose = Pose;
			
			Curve.InitFrom(RequiredBones);

			if (bValidTime)
			{
				if (AnimationSequenceBase->IsValidAdditive())
				{
					CompactPose.ResetToAdditiveIdentity();
					AnimationSequenceBase->GetAnimationPose(PoseData, Context);

					if (EvaluationOptions.bRetrieveAdditiveAsFullPose)
					{
						const UAnimSequence* AnimSequence = Cast<const UAnimSequence>(AnimationSequenceBase);
					
						FBlendedCurve BaseCurve;
						BaseCurve.InitFrom(RequiredBones);
						UE::Anim::FStackAttributeContainer BaseAttributes;
				
						FAnimationPoseData BasePoseData(BasePose, BaseCurve, BaseAttributes);
						AnimSequence->GetAdditiveBasePose(BasePoseData, Context);

						FAnimationRuntime::AccumulateAdditivePose(BasePoseData, PoseData, 1.f, AnimSequence->GetAdditiveAnimType());
						BasePose.NormalizeRotations();
				
						FramePose.SetPose(BasePoseData);
					}
					else
					{
						FramePose.SetPose(PoseData);
					}
				}
				else
				{
					CompactPose.ResetToRefPose();
					AnimationSequenceBase->GetAnimationPose(PoseData, Context);
					FramePose.SetPose(PoseData);
				}
			}
			else
			{
				UE_LOGF(LogAnimationPoseScripting, Warning, "Invalid time value %f for Animation Sequence %ls supplied for GetBonePosesForTime", EvalInterval, *AnimationSequenceBase->GetName());
			}
		}
	}
	else
	{
		UE_LOGF(LogAnimationPoseScripting, Warning, "Invalid Animation Sequence supplied for GetBonePosesForTime");
	}
}

void UAnimPoseExtensions::GetCompactPose(const FAnimPose& Pose, FCompactPose& InOutCompactPose)
{
	Pose.GetPose(InOutCompactPose); 
}

