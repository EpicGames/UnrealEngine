// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationAttributeBlueprintLibrary.h"

#include "Animation/AnimData/AnimDataModel.h"
#include "Animation/AnimData/AttributeIdentifier.h"
#include "Animation/AnimData/IAnimationDataController.h"
#include "Animation/AttributeCurve.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "Containers/ArrayView.h"
#include "Curves/KeyHandle.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Script.h"
#include "UObject/ScriptMacros.h"
#include "UObject/Stack.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationAttributeBlueprintLibrary)

class UObject;

#define LOCTEXT_NAMESPACE "AnimationAttributeBlueprintLibrary"

bool UAnimationAttributeBlueprintLibrary::SetAttributeKey(TScriptInterface<IAnimationDataController> AnimationDataController, const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, const int32& Value)
{
	// We should never hit this! Stubbed to avoid NoExport on the class.
	check(0);
	return false;
}

bool UAnimationAttributeBlueprintLibrary::Generic_SetAttributeKey(TScriptInterface<IAnimationDataController> AnimationDataController, const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, UScriptStruct* ScriptStruct, const void* ValuePtr)
{
	return AnimationDataController->SetAttributeKey(AttributeIdentifier, Time, ValuePtr, ScriptStruct);
}

DEFINE_FUNCTION(UAnimationAttributeBlueprintLibrary::execSetAttributeKey)
{
	P_GET_TINTERFACE(IAnimationDataController, AnimationDataController);
	P_GET_STRUCT_REF(FAnimationAttributeIdentifier, AttributeIdentifier);
	P_GET_PROPERTY(FFloatProperty, TimeInterval);

	Stack.MostRecentProperty = nullptr;
	Stack.MostRecentPropertyAddress = nullptr;
	
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	
	const FStructProperty* ItemProperty = CastField<FStructProperty>(Stack.MostRecentProperty);
	void* ItemDataPtr = Stack.MostRecentPropertyAddress;
	P_FINISH;

	if (!ItemProperty || !ItemDataPtr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			LOCTEXT("SetAttributeKey_InvalidValue", "Failed to resolve the attribute value parameter for SetAttributeKey.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	if (!AnimationDataController.GetObject())
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation, 
			LOCTEXT("SetAttributeKey_InvalidController", "Accessed None attempting to call SetAttributeKey.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	bool bResult = false;

	if (AnimationDataController && ItemProperty)
	{		
		P_NATIVE_BEGIN;
		bResult = Generic_SetAttributeKey(AnimationDataController, AttributeIdentifier, TimeInterval, ItemProperty->Struct, ItemDataPtr);
		P_NATIVE_END;
	}

	*(bool*)RESULT_PARAM = bResult;
}

bool UAnimationAttributeBlueprintLibrary::SetAttributeKeys(TScriptInterface<IAnimationDataController> AnimationDataController, const FAnimationAttributeIdentifier& AttributeIdentifier, const TArray<float>& Times, const TArray<int32>& Values)
{
	// We should never hit this! Stubbed to avoid NoExport on the class.
	check(0);
	return false;
}

bool UAnimationAttributeBlueprintLibrary::Generic_SetAttributeKeys(TScriptInterface<IAnimationDataController> AnimationDataController, const FAnimationAttributeIdentifier& AttributeIdentifier, const TArray<float>& Times, const void* ValuesArray, const FArrayProperty* ValuesArrayProperty)
{	
	FScriptArrayHelper ArrayHelper(ValuesArrayProperty, ValuesArray);

	TArray<const void*> ItemPtrs;
	const int32 NumberOfItems = ArrayHelper.Num();
	ItemPtrs.SetNumZeroed(NumberOfItems);
	for (int32 Index = 0; Index < NumberOfItems; ++Index)
	{
		ItemPtrs[Index] = ArrayHelper.GetRawPtr(Index);
	}

	const FStructProperty* InnerStructProperty = CastField<const FStructProperty>(ValuesArrayProperty->Inner);		
	return AnimationDataController->SetAttributeKeys(AttributeIdentifier, MakeArrayView(Times), MakeArrayView(ItemPtrs), InnerStructProperty->Struct);
}

DEFINE_FUNCTION(UAnimationAttributeBlueprintLibrary::execSetAttributeKeys)
{
	P_GET_TINTERFACE(IAnimationDataController, AnimationDataController);
	P_GET_STRUCT_REF(FAnimationAttributeIdentifier, AttributeIdentifier);
	P_GET_TARRAY_REF(float, TimeIntervals);

	Stack.MostRecentProperty = nullptr;
	Stack.MostRecentPropertyAddress = nullptr;
	
	Stack.StepCompiledIn<FArrayProperty>(nullptr);

	void* ArrayAddr = Stack.MostRecentPropertyAddress;
	FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);	
	P_FINISH;

	if (!ArrayAddr || !ArrayProperty)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			LOCTEXT("SetAttributeKeys_InvalidValue", "Failed to resolve the attribute values parameter for SetAttributeKeys.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	
	if (!AnimationDataController.GetObject())
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation, 
			LOCTEXT("SetAttributeKeys_InvalidController", "Accessed None attempting to call SetAttributeKeys.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	bool bResult = false;

	if (AnimationDataController)
	{		
		P_NATIVE_BEGIN;
		bResult = Generic_SetAttributeKeys(AnimationDataController, AttributeIdentifier, TimeIntervals, ArrayAddr, ArrayProperty);
		P_NATIVE_END;
	}

	*(bool*)RESULT_PARAM = bResult;
}


bool UAnimationAttributeBlueprintLibrary::GetAttributeKey(TScriptInterface<IAnimationDataModel> AnimationDataModel, const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, int32& Value)
{
	// We should never hit this! Stubbed to avoid NoExport on the class.
	check(0);
	return false;
}

bool UAnimationAttributeBlueprintLibrary::Generic_GetAttributeKey(TScriptInterface<IAnimationDataModel> AnimationDataModel, const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, UScriptStruct* ScriptStruct, void* ValuePtr)
{
	const FAnimatedBoneAttribute* AttributePtr = AnimationDataModel->FindAttribute(AttributeIdentifier);

	// Find attribute
	if (AttributePtr && ScriptStruct)
	{
		// And the key on its attribute curve
		const FAttributeCurve& Curve = AttributePtr->Curve;
		check(ScriptStruct == Curve.GetScriptStruct());
		const FKeyHandle KeyHandle = Curve.FindKey(Time);
		if (KeyHandle != FKeyHandle::Invalid())
		{
			// Copy out the value to ptr
			const FAttributeKey& Key = Curve.GetKey(KeyHandle);
			ScriptStruct->CopyScriptStruct(ValuePtr, Key.GetValuePtr<void>(), 1);
			
			return true;
		}
	}

	return false;
}

DEFINE_FUNCTION(UAnimationAttributeBlueprintLibrary::execGetAttributeKey)
{
	P_GET_TINTERFACE(IAnimationDataModel, AnimationDataModel);
	P_GET_STRUCT_REF(FAnimationAttributeIdentifier, AttributeIdentifier);
	P_GET_PROPERTY(FFloatProperty, TimeInterval);

	Stack.MostRecentProperty = nullptr;
	Stack.MostRecentPropertyAddress = nullptr;
	
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	
	const FStructProperty* ItemProperty = CastField<FStructProperty>(Stack.MostRecentProperty);
	void* ItemDataPtr = Stack.MostRecentPropertyAddress;
	P_FINISH;

	if (!ItemProperty || !ItemDataPtr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			LOCTEXT("GetAttributeKey_InvalidValue", "Failed to resolve the attribute value parameter for GetAttributeKey.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	if (!AnimationDataModel.GetObject())
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation, 
			LOCTEXT("GetAttributeKey_InvalidController", "Accessed None attempting to call GetAttributeKey.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	bool bResult = false;

	if (AnimationDataModel && ItemProperty)
	{		
		P_NATIVE_BEGIN;
		bResult = Generic_GetAttributeKey(AnimationDataModel, AttributeIdentifier, TimeInterval, ItemProperty->Struct, ItemDataPtr);
		P_NATIVE_END;
	}

	*(bool*)RESULT_PARAM = bResult;
}


bool UAnimationAttributeBlueprintLibrary::GetAttributeKeys(TScriptInterface<IAnimationDataModel> AnimationDataModel, const FAnimationAttributeIdentifier& AttributeIdentifier, TArray<float>& OutTimes, TArray<int32>& OutValues)
{
	// We should never hit this! Stubbed to avoid NoExport on the class.
	check(0);
	return false;
}

bool UAnimationAttributeBlueprintLibrary::Generic_GetAttributeKeys(TScriptInterface<IAnimationDataModel> AnimationDataModel, const FAnimationAttributeIdentifier& AttributeIdentifier, TArray<float>& Times, void* ValuesArray, const FArrayProperty* ValuesArrayProperty)
{
	const FAnimatedBoneAttribute* AttributePtr = AnimationDataModel->FindAttribute(AttributeIdentifier);

	// Find attribute
	if (AttributePtr && ValuesArrayProperty)
	{
		const FStructProperty* InnerStructProperty = CastField<const FStructProperty>(ValuesArrayProperty->Inner);	
		// And the key on its attribute curve
		const FAttributeCurve& Curve = AttributePtr->Curve;
		check(InnerStructProperty->Struct == Curve.GetScriptStruct());

		const TArray<FAttributeKey>& Keys = Curve.GetConstRefOfKeys();

		
		FScriptArrayHelper ArrayHelper(ValuesArrayProperty, ValuesArray);
		ArrayHelper.Resize(Keys.Num());
		Times.SetNum(Keys.Num());
		
		for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); ++KeyIndex)
		{
			const FAttributeKey& Key = Keys[KeyIndex];
			Times[KeyIndex] = Key.Time;

			uint8* ValuePtr = ArrayHelper.GetRawPtr(KeyIndex);
			InnerStructProperty->Struct->CopyScriptStruct(ValuePtr, Key.GetValuePtr<void>(), 1);
		}
	}

	return false;
}

DEFINE_FUNCTION(UAnimationAttributeBlueprintLibrary::execGetAttributeKeys)
{
	P_GET_TINTERFACE(IAnimationDataModel, AnimationDataModel);
	P_GET_STRUCT_REF(FAnimationAttributeIdentifier, AttributeIdentifier);
	P_GET_TARRAY_REF(float, TimeIntervals);

	Stack.MostRecentProperty = nullptr;
	Stack.MostRecentPropertyAddress = nullptr;
	
	Stack.StepCompiledIn<FArrayProperty>(nullptr);

	void* ArrayAddr = Stack.MostRecentPropertyAddress;
	FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Stack.MostRecentProperty);	
	P_FINISH;

	if (!ArrayAddr || !ArrayProperty)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			LOCTEXT("GetAttributeKeys_InvalidValue", "Failed to resolve the attribute values parameter for GetAttributeKeys.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	
	if (!AnimationDataModel.GetObject())
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation, 
			LOCTEXT("GetAttributeKeys_InvalidController", "Accessed None attempting to call GetAttributeKeys.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	bool bResult = false;

	if (AnimationDataModel)
	{		
		P_NATIVE_BEGIN;
		bResult = Generic_GetAttributeKeys(AnimationDataModel, AttributeIdentifier, TimeIntervals, ArrayAddr, ArrayProperty);
		P_NATIVE_END;
	}

	*(bool*)RESULT_PARAM = bResult;
}

#undef LOCTEXT_NAMESPACE
