// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationBlueprintLibrary.h"

#include "Algo/Transform.h"
#include "AnimGraphNode_AssetPlayerBase.h"
#include "AnimGraphNode_Base.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimBoneCompressionSettings.h"
#include "Animation/AnimCurveCompressionSettings.h"
#include "Animation/VariableFrameStrippingSettings.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Animation/AnimData/AttributeIdentifier.h"
#include "Animation/AnimData/CurveIdentifier.h"
#include "Animation/AnimData/IAnimationDataController.h"
#include "Animation/AnimLinkableElement.h"
#include "Animation/AnimMetaData.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimSequenceHelpers.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimationSettings.h"
#include "Animation/AttributeCurve.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "Animation/CustomAttributes.h"
#include "Animation/Skeleton.h"
#include "AnimationGraph.h"
#include "CommonFrameRates.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "Engine/Blueprint.h"
#include "Engine/SkeletalMesh.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/Guid.h"
#include "Misc/QualifiedFrameTime.h"
#include "Misc/Timecode.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ReferenceSkeleton.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "Animation/AnimData/IAnimationDataModel.h"
#include "AnimPose.h"
#include "UObject/UnrealTypePrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationBlueprintLibrary)

#define LOCTEXT_NAMESPACE "AnimationBlueprintLibrary"


IMPLEMENT_MODULE(IModuleInterface, AnimationBlueprintLibrary);

DEFINE_LOG_CATEGORY_STATIC(LogAnimationBlueprintLibrary, Verbose, All);

static void GetBonePosesForTimeInternal(const UAnimSequenceBase* AnimationSequenceBase, TArray<FName> BoneNames, float Time, bool bExtractRootMotion, TArray<FTransform>& Poses, const USkeletalMesh* PreviewMesh = nullptr)
{
	Poses.Empty(BoneNames.Num());
	if (AnimationSequenceBase && AnimationSequenceBase->GetSkeleton())
	{
		Poses.AddDefaulted(BoneNames.Num());

		if (BoneNames.Num())
		{
			TArray<FName> TrackNames;
			AnimationSequenceBase->GetDataModel()->GetBoneTrackNames(TrackNames);
			
			for (int32 BoneNameIndex = 0; BoneNameIndex < BoneNames.Num(); ++BoneNameIndex)
			{
				const FName& BoneName = BoneNames[BoneNameIndex];
				if (TrackNames.Contains(BoneName))
				{
					FAnimPoseEvaluationOptions EvaluationOptions = FAnimPoseEvaluationOptions();
					FAnimPose AnimPose;
    
					UAnimPoseExtensions::GetAnimPoseAtTime(AnimationSequenceBase, Time, EvaluationOptions, AnimPose);
					Poses[BoneNameIndex] = UAnimPoseExtensions::GetBonePose(AnimPose, BoneName, EAnimPoseSpaces::Local);	
				}
				else
				{
					// otherwise, get ref pose if exists
					const FReferenceSkeleton& RefSkeleton = (PreviewMesh)? PreviewMesh->GetRefSkeleton() : AnimationSequenceBase->GetSkeleton()->GetReferenceSkeleton();
					const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
					if (BoneIndex != INDEX_NONE)
					{
						Poses[BoneNameIndex] = RefSkeleton.GetRefBonePose()[BoneIndex];
					}
					else
					{
						UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid bone name %ls for Animation Sequence %ls supplied for GetBonePosesForTime", *BoneName.ToString(), *AnimationSequenceBase->GetName());
						Poses[BoneNameIndex] = FTransform::Identity;
					}
				}			
			}
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Error, "Invalid or no bone names specified to retrieve poses given Animation Sequence %ls in GetBonePosesForTime", *AnimationSequenceBase->GetName());
		}	
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for GetBonePosesForTime");
	}
}

void UAnimationBlueprintLibrary::GetNumFrames(const UAnimSequenceBase* AnimationSequenceBase, int32& NumFrames)
{
	NumFrames = 0;
	if (AnimationSequenceBase)
	{
		NumFrames = AnimationSequenceBase->GetNumberOfSampledKeys() - 1;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for GetNumFrames");
	}
}

void UAnimationBlueprintLibrary::GetNumKeys(const UAnimSequenceBase* AnimationSequenceBase, int32& NumKeys)
{
	NumKeys = 0;
	if (AnimationSequenceBase)
	{
		NumKeys = AnimationSequenceBase->GetNumberOfSampledKeys();
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for GetNumKeys");
	}
}

void UAnimationBlueprintLibrary::GetAnimationTrackNames(const UAnimSequenceBase* AnimationSequenceBase, TArray<FName>& TrackNames)
{
	TrackNames.Empty();
	if (AnimationSequenceBase)
	{
		AnimationSequenceBase->GetDataModel()->GetBoneTrackNames(TrackNames);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for GetBoneTrackNames");
	}	
}

void UAnimationBlueprintLibrary::GetMontageSlotNames(const UAnimMontage* AnimationMontage, TArray<FName>& SlotNames)
{
	SlotNames.Empty();
	if (AnimationMontage)
	{
		for (int32 SlotIdx = 0; SlotIdx < AnimationMontage->SlotAnimTracks.Num(); SlotIdx++)
		{
			const FSlotAnimationTrack& SlotAnimTrack = AnimationMontage->SlotAnimTracks[SlotIdx];
			SlotNames.Add(SlotAnimTrack.SlotName);
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Montage supplied for GetMontageSlotNames");
	}
}

void UAnimationBlueprintLibrary::GetAnimationCurveNames(const UAnimSequenceBase* AnimationSequenceBase, ERawCurveTrackTypes CurveType, TArray<FName>& CurveNames)
{
	CurveNames.Empty();
	if (AnimationSequenceBase)
	{
		auto GetCurveName = [](const auto& Curve) -> FName
		{
			return Curve.GetName();
		};

		switch (CurveType)
		{
			case ERawCurveTrackTypes::RCT_Float:
			{
				Algo::Transform(AnimationSequenceBase->GetDataModel()->GetFloatCurves(), CurveNames, GetCurveName);
				break;
			}

			case ERawCurveTrackTypes::RCT_Transform:
			{
				Algo::Transform(AnimationSequenceBase->GetDataModel()->GetTransformCurves(), CurveNames, GetCurveName);
				break;
			}

			default:
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid CurveType supplied for GetCurveNames");
			}
		}
		
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for GetCurveNames");
	}
}

FTransform UAnimationBlueprintLibrary::ExtractRootTrackTransform(const UAnimSequenceBase* AnimationSequenceBase, float Time)
{
	if (AnimationSequenceBase == nullptr)
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for ExtractRootTrackTransform");
		return FTransform::Identity;
	}
	
	return AnimationSequenceBase->ExtractRootTrackTransform(FAnimExtractContext(static_cast<double>(Time)), nullptr);
}

void UAnimationBlueprintLibrary::GetBoneCompressionSettings(const UAnimSequence* AnimationSequence, UAnimBoneCompressionSettings*& CompressionSettings)
{
	if (AnimationSequence == nullptr)
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for GetBoneCompressionSettings");
		return;
	}

	CompressionSettings = AnimationSequence->BoneCompressionSettings;
}

void UAnimationBlueprintLibrary::SetBoneCompressionSettings(UAnimSequence* AnimationSequence, UAnimBoneCompressionSettings* CompressionSettings)
{
	if (AnimationSequence == nullptr)
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for SetBoneCompressionSettings");
		return;
	}

	if (CompressionSettings == nullptr || !CompressionSettings->AreSettingsValid())
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Bone Compression Settings supplied for SetBoneCompressionSettings");
		return;
	}

	AnimationSequence->BoneCompressionSettings = CompressionSettings;
}

void UAnimationBlueprintLibrary::GetCurveCompressionSettings(const UAnimSequence* AnimationSequence, UAnimCurveCompressionSettings*& CompressionSettings)
{
	if (AnimationSequence == nullptr)
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for GetCurveCompressionSettings");
		return;
	}

	CompressionSettings = AnimationSequence->CurveCompressionSettings;
}

void UAnimationBlueprintLibrary::SetCurveCompressionSettings(UAnimSequence* AnimationSequence, UAnimCurveCompressionSettings* CompressionSettings)
{
	if (AnimationSequence == nullptr)
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for SetCurveCompressionSettings");
		return;
	}

	if (CompressionSettings == nullptr || !CompressionSettings->AreSettingsValid())
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Bone Compression Settings supplied for SetCurveCompressionSettings");
		return;
	}

	AnimationSequence->CurveCompressionSettings = CompressionSettings;
}

void UAnimationBlueprintLibrary::GetVariableFrameStrippingSettings(const UAnimSequence* AnimationSequence, UVariableFrameStrippingSettings*& VariableFrameStrippingSettings)
{
	if (AnimationSequence == nullptr)
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid VariableFrameStrippingSettings supplied for GetVariableFrameStrippingSettings");
		return;
	}

	VariableFrameStrippingSettings = AnimationSequence->VariableFrameStrippingSettings;
}

void UAnimationBlueprintLibrary::SetVariableFrameStrippingSettings(UAnimSequence* AnimationSequence, UVariableFrameStrippingSettings* VariableFrameStrippingSettings)
{
	if (AnimationSequence == nullptr)
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for SetVariableFrameStrippingSettings");
		return;
	}

	if (VariableFrameStrippingSettings == nullptr)
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid VariableFrameStrippingSettings supplied for SetVariableFrameStrippingSettings");
		return;
	}

	AnimationSequence->VariableFrameStrippingSettings = VariableFrameStrippingSettings;
}

void UAnimationBlueprintLibrary::GetAdditiveAnimationType(const UAnimSequence* AnimationSequence, TEnumAsByte<enum EAdditiveAnimationType>& AdditiveAnimationType)
{
	if (AnimationSequence)
	{
		AdditiveAnimationType = AnimationSequence->AdditiveAnimType;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for GetAdditiveAnimationType");
	}
}

void UAnimationBlueprintLibrary::SetAdditiveAnimationType(UAnimSequence* AnimationSequence, const TEnumAsByte<enum EAdditiveAnimationType> AdditiveAnimationType)
{
	if (AnimationSequence)
	{
		AnimationSequence->AdditiveAnimType = AdditiveAnimationType;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for SetAdditiveAnimationType");
	}
}

void UAnimationBlueprintLibrary::GetAdditiveBasePoseType(const UAnimSequence* AnimationSequence, TEnumAsByte<enum EAdditiveBasePoseType>& AdditiveBasePoseType)
{
	if (AnimationSequence)
	{
		AdditiveBasePoseType = AnimationSequence->RefPoseType;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for GetAdditiveBasePoseType");
	}
}

void UAnimationBlueprintLibrary::SetAdditiveBasePoseType(UAnimSequence* AnimationSequence, const TEnumAsByte<enum EAdditiveBasePoseType> AdditiveBasePoseType)
{
	if (AnimationSequence)
	{
		AnimationSequence->RefPoseType = AdditiveBasePoseType;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for SetAdditiveBasePoseType");
	}
}

void UAnimationBlueprintLibrary::GetAnimationInterpolationType(const UAnimSequence* AnimationSequence, EAnimInterpolationType& InterpolationType)
{
	if (AnimationSequence)
	{
		InterpolationType = AnimationSequence->Interpolation;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for GetAnimationInterpolationType");
	}
}

void UAnimationBlueprintLibrary::SetAnimationInterpolationType(UAnimSequence* AnimationSequence, EAnimInterpolationType Type)
{
	if (AnimationSequence)
	{
		AnimationSequence->Interpolation = Type;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for SetAnimationInterpolationType");
	}
}

bool UAnimationBlueprintLibrary::IsRootMotionEnabled(const UAnimSequence* AnimationSequence)
{
	bool bEnabled = false;
	if (AnimationSequence)
	{
		bEnabled = AnimationSequence->bEnableRootMotion;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for IsRootMotionEnabled");
	}

	return bEnabled;
}

void UAnimationBlueprintLibrary::SetRootMotionEnabled(UAnimSequence* AnimationSequence, bool bEnabled)
{
	if (AnimationSequence)
	{
		AnimationSequence->bEnableRootMotion = bEnabled;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for SetRootMotionEnabled");
	}
}

void  UAnimationBlueprintLibrary::GetRootMotionLockType(const UAnimSequence* AnimationSequence, TEnumAsByte<ERootMotionRootLock::Type>& LockType)
{
	if (AnimationSequence)
	{
		LockType = AnimationSequence->RootMotionRootLock;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for GetRootMotionLockType");
	}
}

void UAnimationBlueprintLibrary::SetRootMotionLockType(UAnimSequence* AnimationSequence, TEnumAsByte<ERootMotionRootLock::Type> RootMotionLockType)
{
	if (AnimationSequence)
	{
		AnimationSequence->RootMotionRootLock = RootMotionLockType;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for SetRootMotionLockType");
	}	
}

bool UAnimationBlueprintLibrary::IsRootMotionLockForced(const UAnimSequence* AnimationSequence)
{
	bool bIsLocked = false;
	if (AnimationSequence)
	{
		bIsLocked = AnimationSequence->bForceRootLock;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for IsRootMotionLockForced");	
	}

	return bIsLocked;
}

void UAnimationBlueprintLibrary::SetIsRootMotionLockForced(UAnimSequence* AnimationSequence, bool bForced)
{
	if (AnimationSequence)
	{
		AnimationSequence->bForceRootLock = bForced;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for SetIsRootMotionLockForced");
	}
}

void UAnimationBlueprintLibrary::GetAnimationSyncMarkers(const UAnimSequence* AnimationSequence, TArray<FAnimSyncMarker>& Markers)
{
	Markers.Empty();
	if (AnimationSequence)
	{
		Markers = AnimationSequence->AuthoredSyncMarkers;;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for GetAnimationSyncMarkers");
	}
}

void UAnimationBlueprintLibrary::GetUniqueMarkerNames(const UAnimSequence* AnimationSequence, TArray<FName>& MarkerNames)
{
	MarkerNames.Empty();
	if (AnimationSequence)
	{
		MarkerNames = AnimationSequence->UniqueMarkerNames;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for GetUniqueMarkerNames");
	}
}

void UAnimationBlueprintLibrary::AddAnimationSyncMarker(UAnimSequence* AnimationSequence, FName MarkerName, float Time, FName TrackName)
{
	if (AnimationSequence)
	{
		const bool bIsValidTrackName = IsValidAnimNotifyTrackName(AnimationSequence, TrackName);
		const bool bIsValidTime = IsValidTimeInternal(AnimationSequence, Time);

		if (bIsValidTrackName && bIsValidTime)
		{
			FAnimSyncMarker NewMarker;
			NewMarker.MarkerName = MarkerName;
			NewMarker.Time = Time;
			NewMarker.TrackIndex = GetTrackIndexForAnimationNotifyTrackName(AnimationSequence, TrackName);
			NewMarker.Guid = FGuid::NewGuid();

			AnimationSequence->AuthoredSyncMarkers.Add(NewMarker);
			AnimationSequence->AnimNotifyTracks[NewMarker.TrackIndex].SyncMarkers.Add(&AnimationSequence->AuthoredSyncMarkers.Last());
						
			AnimationSequence->RefreshSyncMarkerDataFromAuthored();

			// Refresh all cached data
			AnimationSequence->RefreshCacheData();
		}
		else
		{
			if (!bIsValidTrackName)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Animation Notify Track %ls does not exist in Animation Sequence %ls", *TrackName.ToString(), *AnimationSequence->GetName());
			}

			if (!bIsValidTime)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "%f is outside of Animation Sequence %ls range", Time, *AnimationSequence->GetName());
			}
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for AddAnimationSyncMarker");
	}
	
}

bool UAnimationBlueprintLibrary::IsValidAnimationSyncMarkerName(const UAnimSequence* AnimationSequence, FName MarkerName)
{
	bool bIsValid = false;
	if (AnimationSequence)
	{
		bIsValid = AnimationSequence->UniqueMarkerNames.Contains(MarkerName);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for IsValidAnimationSyncMarkerName");
	}

	return bIsValid;	
}

int32 UAnimationBlueprintLibrary::RemoveAnimationSyncMarkersByName(UAnimSequence* AnimationSequence, FName MarkerName)
{
	int32 NumRemovedMarkers = 0;
	if (AnimationSequence)
	{
		NumRemovedMarkers = AnimationSequence->AuthoredSyncMarkers.RemoveAll(
			[&](const FAnimSyncMarker& Marker)
		{
			return Marker.MarkerName == MarkerName;
		});

		AnimationSequence->RefreshSyncMarkerDataFromAuthored();

		// Refresh all cached data
		AnimationSequence->RefreshCacheData();
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for RemoveAnimationSyncMarkersByName");
	}
	
	return NumRemovedMarkers;
}

int32 UAnimationBlueprintLibrary::RemoveAnimationSyncMarkersByTrack(UAnimSequence* AnimationSequence, FName NotifyTrackName)
{
	int32 NumRemovedMarkers = 0;
	if (AnimationSequence)
	{
		const int32 TrackIndex = GetTrackIndexForAnimationNotifyTrackName(AnimationSequence, NotifyTrackName);
		if (TrackIndex != INDEX_NONE)
		{
			NumRemovedMarkers = AnimationSequence->AuthoredSyncMarkers.RemoveAll(
				[&](const FAnimSyncMarker& Marker)
			{
				return Marker.TrackIndex == TrackIndex;
			});

			AnimationSequence->RefreshSyncMarkerDataFromAuthored();

			// Refresh all cached data
			AnimationSequence->RefreshCacheData();
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Animation Notify Track %ls does not exist on Animation Sequence %ls", *NotifyTrackName.ToString(), *AnimationSequence->GetName());
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for RemoveAnimationSyncMarkersByTrack");
	}	
	
	return NumRemovedMarkers;
}

void UAnimationBlueprintLibrary::RemoveAllAnimationSyncMarkers(UAnimSequence* AnimationSequence)
{
	if (AnimationSequence)
	{
		AnimationSequence->AuthoredSyncMarkers.Empty();
		AnimationSequence->RefreshSyncMarkerDataFromAuthored();

		// Refresh all cached data
		AnimationSequence->RefreshCacheData();
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for RemoveAllAnimationSyncMarkers");
	}
}

void UAnimationBlueprintLibrary::GetAnimationNotifyEvents(const UAnimSequenceBase* AnimationSequenceBase, TArray<FAnimNotifyEvent>& NotifyEvents)
{
	NotifyEvents.Empty();
	if (AnimationSequenceBase)
	{
		NotifyEvents = AnimationSequenceBase->Notifies;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for GetAnimationNotifyEvents");
	}	
}

void UAnimationBlueprintLibrary::GetAnimationNotifyEventNames(const UAnimSequenceBase* AnimationSequenceBase, TArray<FName>& EventNames)
{
	EventNames.Empty();
	if (AnimationSequenceBase)
	{
		for (const FAnimNotifyEvent& Event : AnimationSequenceBase->Notifies)
		{
			EventNames.AddUnique(Event.NotifyName);
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for GetAnimationNotifyEventNames");
	}	
}

UAnimNotify* UAnimationBlueprintLibrary::AddAnimationNotifyEvent(UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName, float StartTime, TSubclassOf<UAnimNotify> NotifyClass)
{
	UAnimNotify* Notify = nullptr;
	if (AnimationSequenceBase)
	{
		const bool bIsValidTrackName = IsValidAnimNotifyTrackName(AnimationSequenceBase, NotifyTrackName);
		const bool bIsValidTime = IsValidTimeInternal(AnimationSequenceBase, StartTime);

		if (bIsValidTrackName && bIsValidTime)
		{
			FAnimNotifyEvent& NewEvent = AnimationSequenceBase->Notifies.AddDefaulted_GetRef();

			NewEvent.NotifyName = NAME_None;
			NewEvent.Link(AnimationSequenceBase, StartTime);
			NewEvent.TriggerTimeOffset = GetTriggerTimeOffsetForType(AnimationSequenceBase->CalculateOffsetForNotify(StartTime));
			NewEvent.TrackIndex = GetTrackIndexForAnimationNotifyTrackName(AnimationSequenceBase, NotifyTrackName);
			NewEvent.NotifyStateClass = nullptr;
			NewEvent.Guid = FGuid::NewGuid();

			if (NotifyClass)
			{
				Notify = NewObject<UAnimNotify>(AnimationSequenceBase, NotifyClass, NAME_None, RF_Transactional);
				NewEvent.Notify = Notify;

				// Setup name for new event
				if(NewEvent.Notify)
				{
					NewEvent.NotifyName = FName(*NewEvent.Notify->GetNotifyName());
				}
			}
			else
			{
				NewEvent.Notify = nullptr;
			}

			// Refresh all cached data
			AnimationSequenceBase->RefreshCacheData();
		}
		else
		{
			if (!bIsValidTrackName)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Animation Notify Track %ls does not exist on Animation Sequence %ls", *NotifyTrackName.ToString(), *AnimationSequenceBase->GetName());
			}

			if (!bIsValidTime)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "%f is outside of Animation Sequence %ls range", StartTime, *AnimationSequenceBase->GetName());
			}
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for AddAnimationNotifyEvent");
	}

	return Notify;
}

UAnimNotify* UAnimationBlueprintLibrary::AddAnimationNotifyEventFromSource(UAnimSequenceBase* AnimationSequenceBase, float StartTime, FAnimNotifyEvent NotifyEvent, FName NotifyTrackName)
{
	UAnimNotify* ResNotify = nullptr;
	
	if (AnimationSequenceBase)
	{
		TObjectPtr<UAnimNotify> Notify = NotifyEvent.Notify;
		const bool bValidNotify = Notify != nullptr;
		const bool bValidNotifyState = NotifyEvent.NotifyStateClass != nullptr;
		const bool bValidOuter = bValidNotify && Notify->GetOuter() == AnimationSequenceBase;
		const bool bIsValidTrackName = IsValidAnimNotifyTrackName(AnimationSequenceBase, NotifyTrackName);
		const bool bIsValidTime = IsValidTimeInternal(AnimationSequenceBase, StartTime);
		
		if (bValidNotify && bIsValidTrackName && bIsValidTime && !bValidNotifyState)
		{
			// Our actual new event we will copy.
			FAnimNotifyEvent& NewEvent = AnimationSequenceBase->Notifies.AddDefaulted_GetRef();

			// Copy over only blueprint exposed types (to support for adding info such as trigger weight threshold, filtering type, etc).
			const UScriptStruct* ScriptStruct = FAnimNotifyEvent::StaticStruct();
			for (TFieldIterator<FProperty> PropIt(ScriptStruct); PropIt; ++PropIt)
			{
				FProperty* Property = *PropIt;
				if (Property->HasAnyPropertyFlags(CPF_BlueprintVisible | CPF_Edit))
				{
					Property->CopyCompleteValue_InContainer(&NewEvent, &NotifyEvent);
				}
			}

			// Duplicate notify since its outer is not the target animation sequence.
			if (!bValidOuter)
			{
				UAnimNotify* DuplicatedNotify = static_cast<UAnimNotify*>(StaticDuplicateObject(Notify, AnimationSequenceBase, NAME_None, RF_AllFlags, nullptr, EDuplicateMode::Normal, ~EInternalObjectFlags::RootSet));
				check(DuplicatedNotify && DuplicatedNotify->CanBePlaced(AnimationSequenceBase));
				
				NewEvent.Notify = DuplicatedNotify;
			}
			else
			{
				NewEvent.Notify = Notify;
			}

			// Now we have our final notify instance.
			ResNotify = NewEvent.Notify;
			
			// Update info on new notify.
			NewEvent.NotifyName = NAME_None;
			NewEvent.Link(AnimationSequenceBase, StartTime);
			NewEvent.TriggerTimeOffset = GetTriggerTimeOffsetForType(AnimationSequenceBase->CalculateOffsetForNotify(StartTime));
			NewEvent.TrackIndex = GetTrackIndexForAnimationNotifyTrackName(AnimationSequenceBase, NotifyTrackName);
			NewEvent.NotifyStateClass = nullptr;
			NewEvent.Guid = FGuid::NewGuid();		

			// Refresh all cached data
			AnimationSequenceBase->MarkPackageDirty();
			AnimationSequenceBase->RefreshCacheData();
		}
		else
		{
			if (!bValidNotify)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Notify in AddAnimationNotifyEventObject");
			}
			
			if (bValidNotifyState)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Adding Animation Notify States is not supported yet.")
			}
			
			if (!bIsValidTrackName)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Animation Notify Track %ls does not exist on Animation Sequence %ls", *NotifyTrackName.ToString(), *AnimationSequenceBase->GetName());
			}

			if (!bIsValidTime)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "%f is outside of Animation Sequence %ls range", StartTime, *AnimationSequenceBase->GetName());
			}
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for AddAnimationNotifyEventObject");
	}

	return ResNotify;
}

UAnimNotifyState* UAnimationBlueprintLibrary::AddAnimationNotifyStateEvent(UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName, float StartTime, float Duration, TSubclassOf<UAnimNotifyState> NotifyStateClass)
{
	UAnimNotifyState* NotifyState = nullptr;
	if (AnimationSequenceBase)
	{
		const bool bIsValidTrackName = IsValidAnimNotifyTrackName(AnimationSequenceBase, NotifyTrackName);
		const bool bIsValidTime = IsValidTimeInternal(AnimationSequenceBase, StartTime);

		if (bIsValidTrackName && bIsValidTime)
		{
			FAnimNotifyEvent& NewEvent = AnimationSequenceBase->Notifies.AddDefaulted_GetRef();

			NewEvent.NotifyName = NAME_None;
			NewEvent.Link(AnimationSequenceBase, StartTime);
			NewEvent.TriggerTimeOffset = GetTriggerTimeOffsetForType(AnimationSequenceBase->CalculateOffsetForNotify(StartTime));
			NewEvent.TrackIndex = GetTrackIndexForAnimationNotifyTrackName(AnimationSequenceBase, NotifyTrackName);
			NewEvent.Notify = nullptr;
			NewEvent.Guid = FGuid::NewGuid();

			if (NotifyStateClass)
			{
				NotifyState = NewObject<UAnimNotifyState>(AnimationSequenceBase, NotifyStateClass, NAME_None, RF_Transactional);
				NewEvent.NotifyStateClass = NotifyState;

				// Setup name and duration for new event
				if (NewEvent.NotifyStateClass)
				{
					NewEvent.NotifyName = FName(*NewEvent.NotifyStateClass->GetNotifyName());
					NewEvent.SetDuration(Duration);
					NewEvent.EndLink.Link(AnimationSequenceBase, NewEvent.EndLink.GetTime());
				}
			}
			else
			{
				NewEvent.NotifyStateClass = nullptr;
			}

			// Refresh all cached data
			AnimationSequenceBase->RefreshCacheData();			
		}
		else
		{
			if (!bIsValidTrackName)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Animation Notify Track %ls does not exist on Animation Sequence %ls", *NotifyTrackName.ToString(), *AnimationSequenceBase->GetName());
			}

			if (!bIsValidTime)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "%f is outside of Animation Sequence %ls range", StartTime, *AnimationSequenceBase->GetName());
			}
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for AddAnimationNotifyStateEvent");
	}

	return NotifyState;
}

void UAnimationBlueprintLibrary::AddAnimationNotifyEventObject(UAnimSequenceBase* AnimationSequenceBase, float StartTime, UAnimNotify* Notify, FName NotifyTrackName)
{
	if (AnimationSequenceBase)
	{
		const bool bValidNotify = Notify != nullptr;
		const bool bValidOuter = bValidNotify && Notify->GetOuter() == AnimationSequenceBase;
		const bool bIsValidTrackName = IsValidAnimNotifyTrackName(AnimationSequenceBase, NotifyTrackName);
		const bool bIsValidTime = IsValidTimeInternal(AnimationSequenceBase, StartTime);

		if (bValidNotify && bValidOuter && bIsValidTrackName && bIsValidTime)
		{
			FAnimNotifyEvent& NewEvent = AnimationSequenceBase->Notifies.AddDefaulted_GetRef();

			NewEvent.NotifyName = NAME_None;
			NewEvent.Link(AnimationSequenceBase, StartTime);
			NewEvent.TriggerTimeOffset = GetTriggerTimeOffsetForType(AnimationSequenceBase->CalculateOffsetForNotify(StartTime));
			NewEvent.TrackIndex = GetTrackIndexForAnimationNotifyTrackName(AnimationSequenceBase, NotifyTrackName);
			NewEvent.NotifyStateClass = nullptr;
			NewEvent.Notify = Notify;
			NewEvent.Guid = FGuid::NewGuid();		

			// Refresh all cached data
			AnimationSequenceBase->RefreshCacheData();
		}
		else
		{
			if (!bValidNotify)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Notify in AddAnimationNotifyEventObject");
			}

			if (!bValidOuter)
			{
				const FString NotifyName = bValidNotify ? Notify->GetName() : "Invalid Notify";
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Animation Notify %ls Outer is not %ls", *NotifyName, *AnimationSequenceBase->GetName());
			}

			if (!bIsValidTrackName)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Animation Notify Track %ls does not exist on Animation Sequence %ls", *NotifyTrackName.ToString(), *AnimationSequenceBase->GetName());
			}

			if (!bIsValidTime)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "%f is outside of Animation Sequence %ls range", StartTime, *AnimationSequenceBase->GetName());
			}
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for AddAnimationNotifyEventObject");
	}
}

void UAnimationBlueprintLibrary::AddAnimationNotifyStateEventObject(UAnimSequenceBase* AnimationSequenceBase, float StartTime, float Duration, UAnimNotifyState* NotifyState, FName NotifyTrackName)
{
	if (AnimationSequenceBase)
	{
		const bool bValidNotifyState = NotifyState != nullptr;
		const bool bValidOuter = bValidNotifyState && NotifyState->GetOuter() == AnimationSequenceBase;
		const bool bIsValidTrackName = IsValidAnimNotifyTrackName(AnimationSequenceBase, NotifyTrackName);
		const bool bIsValidTime = IsValidTimeInternal(AnimationSequenceBase, StartTime);

		if (bValidNotifyState && bValidOuter && bIsValidTrackName && bIsValidTime)
		{
			FAnimNotifyEvent& NewEvent = AnimationSequenceBase->Notifies.AddDefaulted_GetRef();

			NewEvent.NotifyName = NAME_None;
			NewEvent.Link(AnimationSequenceBase, StartTime);
			NewEvent.TriggerTimeOffset = GetTriggerTimeOffsetForType(AnimationSequenceBase->CalculateOffsetForNotify(StartTime));
			NewEvent.TrackIndex = GetTrackIndexForAnimationNotifyTrackName(AnimationSequenceBase, NotifyTrackName);
			NewEvent.NotifyStateClass = NotifyState;
			NewEvent.Notify = nullptr;
			NewEvent.SetDuration(Duration);
			NewEvent.EndLink.Link(AnimationSequenceBase, NewEvent.EndLink.GetTime());
			NewEvent.Guid = FGuid::NewGuid();

			// Refresh all cached data
			AnimationSequenceBase->RefreshCacheData();
		}
		else
		{
			if (!bValidNotifyState)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Notify in AddAnimationNotifyEventObject");
			}

			if (!bValidOuter)
			{
				const FString NotifyName = bValidNotifyState ? NotifyState->GetName() : "Invalid Notify";
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Animation Notify %ls Outer is not %ls", *NotifyName, *AnimationSequenceBase->GetName());
			}

			if (!bIsValidTrackName)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Animation Notify Track %ls does not exist on Animation Sequence %ls", *NotifyTrackName.ToString(), *AnimationSequenceBase->GetName());
			}

			if (!bIsValidTime)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "%f is outside of Animation Sequence %ls range", StartTime, *AnimationSequenceBase->GetName());
			}
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for AddAnimationNotifyEventObject");
	}
}

static void ReplaceAnimNotifies_Helper(UAnimSequenceBase* AnimationSequence, UClass* OldNotifyClass, UClass* NewNotifyClass, FOnNotifyReplaced OnNotifyReplaced, FOnNotifyStateReplaced OnNotifyStateReplaced)
{
	if (AnimationSequence)
	{
		if (OldNotifyClass != nullptr && NewNotifyClass != nullptr)
		{
			bool bModified = false;
			for(int32 NotifyIndex = 0; NotifyIndex < AnimationSequence->Notifies.Num(); ++NotifyIndex)
			{
				FAnimNotifyEvent& NotifyEvent = AnimationSequence->Notifies[NotifyIndex];

				if ((NotifyEvent.Notify && NotifyEvent.Notify->GetClass() == OldNotifyClass) || 
					(NotifyEvent.NotifyStateClass && NotifyEvent.NotifyStateClass->GetClass() == OldNotifyClass))
				{

					bModified = true;

					// Copy relevant data from the old notify
					float StartTime = NotifyEvent.GetTime();
					float Length = NotifyEvent.GetDuration();
					int32 TargetTrackIndex = NotifyEvent.TrackIndex;
					float TriggerTimeOffset = NotifyEvent.TriggerTimeOffset;
					float EndTriggerTimeOffset = NotifyEvent.EndTriggerTimeOffset;
					int32 SlotIndex = NotifyEvent.GetSlotIndex();
					int32 EndSlotIndex = NotifyEvent.EndLink.GetSlotIndex();
					int32 SegmentIndex = NotifyEvent.GetSegmentIndex();
					int32 EndSegmentIndex = NotifyEvent.GetSegmentIndex();
					EAnimLinkMethod::Type LinkMethod = NotifyEvent.GetLinkMethod();
					EAnimLinkMethod::Type EndLinkMethod = NotifyEvent.EndLink.GetLinkMethod();
					UAnimNotify* OldNotify = NotifyEvent.Notify;
					UAnimNotifyState* OldNotifyState = NotifyEvent.NotifyStateClass;

					// Remove old notify
					AnimationSequence->Notifies.RemoveAt(NotifyIndex, EAllowShrinking::No);

					// Add new notify in old notifies place
					AnimationSequence->Notifies.InsertDefaulted(NotifyIndex);
					FAnimNotifyEvent& NewEvent = AnimationSequence->Notifies[NotifyIndex];

					// Setup new notify
					NewEvent.NotifyName = NAME_None;
					NewEvent.Link(AnimationSequence, StartTime);
					NewEvent.TriggerTimeOffset = TriggerTimeOffset;
					NewEvent.TrackIndex = TargetTrackIndex;
					NewEvent.ChangeSlotIndex(SlotIndex);
					NewEvent.SetSegmentIndex(SegmentIndex);
					NewEvent.ChangeLinkMethod(LinkMethod);
					NewEvent.Guid = FGuid::NewGuid();

					UObject* AnimNotifyClass = NewObject<UObject>(AnimationSequence, NewNotifyClass, NAME_None, RF_Transactional);
					NewEvent.NotifyStateClass = Cast<UAnimNotifyState>(AnimNotifyClass);
					NewEvent.Notify = Cast<UAnimNotify>(AnimNotifyClass);

					// Setup name and duration for new event
					if (NewEvent.NotifyStateClass)
					{
						NewEvent.NotifyName = FName(*NewEvent.NotifyStateClass->GetNotifyName());
						NewEvent.SetDuration(Length);
						NewEvent.EndTriggerTimeOffset = EndTriggerTimeOffset;
						NewEvent.EndLink.ChangeSlotIndex(EndSlotIndex);
						NewEvent.EndLink.SetSegmentIndex(EndSegmentIndex);
						NewEvent.EndLink.ChangeLinkMethod(EndLinkMethod);

						OnNotifyStateReplaced.ExecuteIfBound(OldNotifyState, NewEvent.NotifyStateClass);
					}
					else if(NewEvent.Notify)
					{
						NewEvent.NotifyName = FName(*NewEvent.Notify->GetNotifyName());

						OnNotifyReplaced.ExecuteIfBound(OldNotify, NewEvent.Notify);
					}

					NewEvent.Update();
				}
			}

			if(bModified)
			{
				// Refresh all cached data
				AnimationSequence->MarkPackageDirty();
				AnimationSequence->RefreshCacheData();
			}
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Notify Class for ReplaceAnimNotifies");
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for ReplaceAnimNotifies");
	}
}

void UAnimationBlueprintLibrary::ReplaceAnimNotifyStates(UAnimSequenceBase* AnimationSequenceBase, TSubclassOf<UAnimNotifyState> OldNotifyClass, TSubclassOf<UAnimNotifyState> NewNotifyClass, FOnNotifyStateReplaced OnNotifyStateReplaced)
{
	ReplaceAnimNotifies_Helper(AnimationSequenceBase, OldNotifyClass.Get(), NewNotifyClass.Get(), FOnNotifyReplaced(), OnNotifyStateReplaced);
}

void UAnimationBlueprintLibrary::ReplaceAnimNotifies(UAnimSequenceBase* AnimationSequenceBase, TSubclassOf<UAnimNotify> OldNotifyClass, TSubclassOf<UAnimNotify> NewNotifyClass, FOnNotifyReplaced OnNotifyReplaced)
{
	ReplaceAnimNotifies_Helper(AnimationSequenceBase, OldNotifyClass.Get(), NewNotifyClass.Get(), OnNotifyReplaced, FOnNotifyStateReplaced());
}

void UAnimationBlueprintLibrary::CopyAnimNotifiesFromSequence(UAnimSequenceBase* SourceAnimationSequenceBase, UAnimSequenceBase* DestinationAnimSequenceBase, bool bDeleteExistingNotifies)
{
	if (SourceAnimationSequenceBase == nullptr)
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Source Animation Sequence for CopyAnimNotifiesFromSequence");
	}
	else if (DestinationAnimSequenceBase == nullptr)
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Destination Animation Sequence for CopyAnimNotifiesFromSequence");
	}
	else if (SourceAnimationSequenceBase == DestinationAnimSequenceBase)
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Source and Destination Animation Sequence are the same for CopyAnimNotifiesFromSequence");
	}
	else
	{
		const bool bShowDialogs = false;
		UE::Anim::CopyNotifies(SourceAnimationSequenceBase, DestinationAnimSequenceBase, bShowDialogs, bDeleteExistingNotifies);
	}
}

int32 UAnimationBlueprintLibrary::RemoveAnimationNotifyEventsByName(UAnimSequenceBase* AnimationSequenceBase, FName NotifyName)
{
	int32 NumRemovedEvents = 0;
	if (AnimationSequenceBase)
	{
		NumRemovedEvents = AnimationSequenceBase->Notifies.RemoveAll(
			[&](const FAnimNotifyEvent& Event)
		{
			return Event.NotifyName == NotifyName;
		});

		// Refresh all cached data
		AnimationSequenceBase->RefreshCacheData();
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for RemoveAnimationNotifyEventsByName");
	}	

	return NumRemovedEvents;
}

int32 UAnimationBlueprintLibrary::RemoveAnimationNotifyEventsByTrack(UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName)
{
	int32 NumRemovedEvents = 0;
	if (AnimationSequenceBase)
	{
		const bool bIsValidTrackName = IsValidAnimNotifyTrackName(AnimationSequenceBase, NotifyTrackName);
		if (bIsValidTrackName)
		{
			const int32 TrackIndex = GetTrackIndexForAnimationNotifyTrackName(AnimationSequenceBase, NotifyTrackName);
			NumRemovedEvents = AnimationSequenceBase->Notifies.RemoveAll(
				[&](const FAnimNotifyEvent& Event)
			{
				return Event.TrackIndex == TrackIndex;
			});

			// Refresh all cached data
			AnimationSequenceBase->RefreshCacheData();
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Animation Notify Track %ls does not exist on Animation Sequence %ls", *NotifyTrackName.ToString(), *AnimationSequenceBase->GetName());
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for RemoveAnimationNotifyEventsByTrack");
	}
	

	return NumRemovedEvents;
}

void UAnimationBlueprintLibrary::GetAnimationNotifyTrackNames(const UAnimSequenceBase* AnimationSequenceBase, TArray<FName>& TrackNames)
{
	TrackNames.Empty();
	if (AnimationSequenceBase)
	{
		for (const FAnimNotifyTrack& Track : AnimationSequenceBase->AnimNotifyTracks)
		{
			TrackNames.AddUnique(Track.TrackName);
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for GetAnimationNotifyTrackNames");
	}
}

void UAnimationBlueprintLibrary::AddAnimationNotifyTrack(UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName, FLinearColor TrackColor /*= FLinearColor::White*/)
{
	if (AnimationSequenceBase)
	{
		const bool bExistingTrackName = IsValidAnimNotifyTrackName(AnimationSequenceBase, NotifyTrackName);
		if (!bExistingTrackName)
		{
			FAnimNotifyTrack NewTrack;
			NewTrack.TrackName = NotifyTrackName;
			NewTrack.TrackColor = TrackColor;
			AnimationSequenceBase->AnimNotifyTracks.Add(NewTrack);

			// Refresh all cached data
			AnimationSequenceBase->RefreshCacheData();
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Animation Notify Track %ls already exists on Animation Sequence %ls", *NotifyTrackName.ToString(), *AnimationSequenceBase->GetName());
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for AddAnimationNotifyTrack");
	}
	
}

void UAnimationBlueprintLibrary::RemoveAnimationNotifyTrack(UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName)
{
	if (AnimationSequenceBase)
	{
		const int32 TrackIndexToDelete = GetTrackIndexForAnimationNotifyTrackName(AnimationSequenceBase, NotifyTrackName);
		if (TrackIndexToDelete != INDEX_NONE)
		{	
			// Remove all notifies and sync markers on the to-delete-track
			AnimationSequenceBase->Notifies.RemoveAll([&](const FAnimNotifyEvent& Notify) { return Notify.TrackIndex == TrackIndexToDelete; });

			// Before track removal, make sure everything behind is fixed
			for (FAnimNotifyEvent& Notify : AnimationSequenceBase->Notifies)
			{
				if (Notify.TrackIndex > TrackIndexToDelete)
				{
					Notify.TrackIndex = Notify.TrackIndex - 1;
				}				
			}

			if (UAnimSequence* AnimationSequence = Cast<UAnimSequence>(AnimationSequenceBase))
			{
				AnimationSequence->AuthoredSyncMarkers.RemoveAll([&](const FAnimSyncMarker& Marker) { return Marker.TrackIndex == TrackIndexToDelete; });
			for (FAnimSyncMarker& SyncMarker : AnimationSequence->AuthoredSyncMarkers)
			{
				if (SyncMarker.TrackIndex > TrackIndexToDelete)
				{
					SyncMarker.TrackIndex = SyncMarker.TrackIndex - 1;
				}
			}
			}			
			
			// Delete the track itself
			AnimationSequenceBase->AnimNotifyTracks.RemoveAt(TrackIndexToDelete);

			// Refresh all cached data
			AnimationSequenceBase->RefreshCacheData();
		}		
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for RemoveAnimationNotifyTrack");
	}
}

void UAnimationBlueprintLibrary::RemoveAllAnimationNotifyTracks(UAnimSequenceBase* AnimationSequenceBase)
{
	if (AnimationSequenceBase)
{
		AnimationSequenceBase->Notifies.Empty();
		if (UAnimSequence* AnimationSequence = Cast<UAnimSequence>(AnimationSequenceBase))
	{
		AnimationSequence->AuthoredSyncMarkers.Empty();
		}

		// Remove all but one notify tracks
		AnimationSequenceBase->AnimNotifyTracks.SetNum(1);

		// Also remove all stale notifies and sync markers from only track 
		AnimationSequenceBase->AnimNotifyTracks[0].Notifies.Empty();
		AnimationSequenceBase->AnimNotifyTracks[0].SyncMarkers.Empty();

		// Refresh all cached data
		AnimationSequenceBase->RefreshCacheData();
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for RemoveAllAnimationNotifyTracks");
	}	
}

bool UAnimationBlueprintLibrary::IsValidAnimNotifyTrackName(const UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName)
{
	bool bIsValid = false;
	if (AnimationSequenceBase)
	{
		bIsValid = GetTrackIndexForAnimationNotifyTrackName(AnimationSequenceBase, NotifyTrackName) != INDEX_NONE;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for IsValidAnimNotifyTrackName");
	}

	return bIsValid;	
}

int32 UAnimationBlueprintLibrary::GetTrackIndexForAnimationNotifyTrackName(const UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName)
{
	return AnimationSequenceBase->AnimNotifyTracks.IndexOfByPredicate(
		[&](const FAnimNotifyTrack& Track)
	{
		return Track.TrackName == NotifyTrackName;
	});
}

const FAnimNotifyTrack& UAnimationBlueprintLibrary::GetNotifyTrackByName(const UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName)
{
	const int32 TrackIndex = GetTrackIndexForAnimationNotifyTrackName(AnimationSequenceBase, NotifyTrackName);
	checkf(TrackIndex != INDEX_NONE, TEXT("Notify Track %s does not exist on %s"), *NotifyTrackName.ToString(), *AnimationSequenceBase->GetName());
	return AnimationSequenceBase->AnimNotifyTracks[TrackIndex];
}

float UAnimationBlueprintLibrary::GetAnimNotifyEventTriggerTime(const FAnimNotifyEvent& NotifyEvent)
{
	return NotifyEvent.GetTriggerTime();
}

float UAnimationBlueprintLibrary::GetAnimNotifyEventDuration(const FAnimNotifyEvent& NotifyEvent)
{
	return NotifyEvent.Duration;
}

void UAnimationBlueprintLibrary::GetAnimationSyncMarkersForTrack(const UAnimSequence* AnimationSequence, FName NotifyTrackName, TArray<FAnimSyncMarker>& Markers)
{
	Markers.Empty();
	if (AnimationSequence)
	{
		const bool bIsValidTrackName = IsValidAnimNotifyTrackName(AnimationSequence, NotifyTrackName);

		if (bIsValidTrackName)
		{
			const FAnimNotifyTrack& Track = GetNotifyTrackByName(AnimationSequence, NotifyTrackName);
			Markers.Empty(Track.SyncMarkers.Num());
			for (FAnimSyncMarker* Marker : Track.SyncMarkers)
			{
				Markers.Add(*Marker);
			}
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Animation Notify Track %ls does not exist on Animation Sequence %ls", *NotifyTrackName.ToString(), *AnimationSequence->GetName());
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for AddVectorCurveKey");
	}
}

void UAnimationBlueprintLibrary::GetAnimationNotifyEventsForTrack(const UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName, TArray<FAnimNotifyEvent>& Events)
{
	Events.Empty();
	if (AnimationSequenceBase)
	{
		const bool bIsValidTrackName = IsValidAnimNotifyTrackName(AnimationSequenceBase, NotifyTrackName);

		if (bIsValidTrackName)
		{
			const FAnimNotifyTrack& Track = GetNotifyTrackByName(AnimationSequenceBase, NotifyTrackName);
			Events.Empty(Track.Notifies.Num());
			for (FAnimNotifyEvent* Event : Track.Notifies)
			{
				Events.Add(*Event);
			}
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Animation Notify Track %ls does not exist on Animation Sequence %ls", *NotifyTrackName.ToString(), *AnimationSequenceBase->GetName());
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for AddVectorCurveKey");
	}
}

void UAnimationBlueprintLibrary::AddCurve(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, ERawCurveTrackTypes CurveType /*= RCT_Float*/, bool bMetaDataCurve /*= false*/)
{
	if (AnimationSequenceBase)
	{
		static const ESmartNameContainerType ContainerForCurveType[(int32)ERawCurveTrackTypes::RCT_MAX] = { ESmartNameContainerType::SNCT_CurveMapping, ESmartNameContainerType::SNCT_CurveMapping, ESmartNameContainerType::SNCT_TrackCurveMapping };
		const ESmartNameContainerType CurveContainer = ContainerForCurveType[(int32)CurveType];
		const int32 CurveFlags = bMetaDataCurve ? AACF_Metadata : AACF_DefaultCurve;
		
		// Validate combination of curve types

		// Only Float metadata curves are valid
		const bool bValidMetaData = !bMetaDataCurve || (bMetaDataCurve && CurveType == ERawCurveTrackTypes::RCT_Float);
		// Transform curves can only be added if the curve name exists as a bone on the skeleton
		const bool bValidTransformCurveData = CurveType != ERawCurveTrackTypes::RCT_Transform || (AnimationSequenceBase->GetSkeleton() && DoesBoneNameExistInternal(AnimationSequenceBase->GetSkeleton(), CurveName));

		if (bValidMetaData && bValidTransformCurveData )
		{
			// Add or retrieve the smartname
			const bool bCurveAdded = AddCurveInternal(AnimationSequenceBase, CurveName, CurveFlags, CurveType);

			if (!bCurveAdded)
			{
				// Curve already existed
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Curve %ls already exists on the Skeleton %ls.", *CurveName.ToString(), *AnimationSequenceBase->GetSkeleton()->GetName());
			}
		}
		else
		{
			if (!bValidMetaData)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Curve type to be create as metadata, currently only float curves are supported as metadata.");
			}
			
			if (!bValidTransformCurveData)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Transform Curve name, the supplied name %ls does not exist on the Skeleton %ls.", *CurveName.ToString(), AnimationSequenceBase->GetSkeleton() ? *AnimationSequenceBase->GetSkeleton()->GetName() : TEXT("Invalid Skeleton"));
			}
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for AddCurve");
	}	
}

void UAnimationBlueprintLibrary::RemoveCurve(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, bool bRemoveNameFromSkeleton /*= false*/)
{
	if (AnimationSequenceBase)
	{
		const ERawCurveTrackTypes CurveType = RetrieveCurveTypeForCurve(AnimationSequenceBase, CurveName);
		if (CurveType != ERawCurveTrackTypes::RCT_MAX)
		{
			const bool bCurveRemoved = RemoveCurveInternal(AnimationSequenceBase, CurveName, CurveType);
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Could not find SmartName Container for Curve Name %ls while trying to remove the curve", *CurveName.ToString());
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for RemoveCurve");
	}
}

void UAnimationBlueprintLibrary::RemoveAllCurveData(UAnimSequenceBase* AnimationSequenceBase)
{
	if (AnimationSequenceBase)
	{
		IAnimationDataController& Controller = AnimationSequenceBase->GetController();

		Controller.RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Float);
		Controller.RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Transform);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for RemoveAllCurveData");
	}
}

void UAnimationBlueprintLibrary::AddTransformationCurveKey(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, const float Time, const FTransform& Transform)
{
	if (AnimationSequenceBase)
	{
		TArray<float> TimeArray;
		TArray<FTransform> TransformArray;

		TimeArray.Add(Time);
		TransformArray.Add(Transform);

		AddCurveKeysInternal<FTransform, FTransformCurve, ERawCurveTrackTypes::RCT_Transform>(AnimationSequenceBase, CurveName, TimeArray, TransformArray);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for AddTransformationCurveKey");
	}

}

void UAnimationBlueprintLibrary::AddTransformationCurveKeys(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, const TArray<float>& Times, const TArray<FTransform>& Transforms)
{
	if (AnimationSequenceBase)
	{
		if (Times.Num() == Transforms.Num())
		{
			AddCurveKeysInternal<FTransform, FTransformCurve, ERawCurveTrackTypes::RCT_Transform>(AnimationSequenceBase, CurveName, Times, Transforms);
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Number of Time values %i does not match the number of Transforms %i in AddTransformationCurveKeys", Times.Num(), Transforms.Num());
		}	
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for AddTransformationCurveKeys");
	}
}


void UAnimationBlueprintLibrary::AddFloatCurveKey(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, const float Time, const float Value)
{
	if (AnimationSequenceBase)
	{
		TArray<float> TimeArray;
		TArray<float> ValueArray;

		TimeArray.Add(Time);
		ValueArray.Add(Value);

		AddCurveKeysInternal<float, FFloatCurve, ERawCurveTrackTypes::RCT_Float>(AnimationSequenceBase, CurveName, TimeArray, ValueArray);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for AddFloatCurveKey");
	}

}

void UAnimationBlueprintLibrary::AddFloatCurveKeys(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, const TArray<float>& Times, const TArray<float>& Values)
{
	if (AnimationSequenceBase)
	{
		if (Times.Num() == Values.Num())
		{
			AddCurveKeysInternal<float, FFloatCurve, ERawCurveTrackTypes::RCT_Float>(AnimationSequenceBase, CurveName, Times, Values);
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Number of Time values %i does not match the number of Values %i in AddFloatCurveKeys", Times.Num(), Values.Num());			
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for AddFloatCurveKeys");
	}

	
}

void UAnimationBlueprintLibrary::AddVectorCurveKey(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, const float Time, const FVector Vector)
{
	if (AnimationSequenceBase)
	{
		TArray<float> TimeArray;
		TArray<FVector> VectorArray;

		TimeArray.Add(Time);
		VectorArray.Add(Vector);

		AddCurveKeysInternal<FVector, FVectorCurve, ERawCurveTrackTypes::RCT_Vector>(AnimationSequenceBase, CurveName, TimeArray, VectorArray);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for AddVectorCurveKey");
	}

}

void UAnimationBlueprintLibrary::AddVectorCurveKeys(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, const TArray<float>& Times, const TArray<FVector>& Vectors)
{
	if (AnimationSequenceBase)
	{
		if (Times.Num() == Vectors.Num())
		{
			AddCurveKeysInternal<FVector, FVectorCurve, ERawCurveTrackTypes::RCT_Vector>(AnimationSequenceBase, CurveName, Times, Vectors);
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Number of Time values %i does not match the number of Vectors %i in AddVectorCurveKeys", Times.Num(), Vectors.Num());
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for AddVectorCurveKeys");
	}
}

static void SetControllerCurveKeys(IAnimationDataController& Controller, FName Name, const TArray<float>& Times, const TArray<float>& Values)
{
	const int32 NumKeys = Values.Num();
	const FAnimationCurveIdentifier CurveId(Name, ERawCurveTrackTypes::RCT_Float);
	for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
	{
		Controller.SetCurveKey(CurveId, { Times[KeyIndex], Values[KeyIndex] });
	}
}

static void SetControllerCurveKeys(IAnimationDataController& Controller, FName Name, const TArray<float>& Times, const TArray<FTransform>& Values)
{
	const int32 NumKeys = Values.Num();
	const FAnimationCurveIdentifier CurveId(Name, ERawCurveTrackTypes::RCT_Transform);
	for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
	{
		Controller.SetTransformCurveKey(CurveId, Times[KeyIndex], Values[KeyIndex]);
	}
}
	
static void SetControllerCurveKeys(IAnimationDataController& Controller, FName Name, const TArray<float>& Times, const TArray<FVector>& Values)
{
	ensure(false);
}

template <typename DataType, typename CurveClass, ERawCurveTrackTypes CurveType>
void UAnimationBlueprintLibrary::AddCurveKeysInternal(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, const TArray<float>& Times, const TArray<DataType>& KeyData)
{
	checkf(Times.Num() == KeyData.Num(), TEXT("Not enough key data supplied"));

	// Retrieve the curve by name
	const FAnimationCurveIdentifier CurveId(CurveName, CurveType);
	const CurveClass* Curve = static_cast<const CurveClass*>(AnimationSequenceBase->GetDataModel()->FindCurve(CurveId));
	if (Curve)
	{
		IAnimationDataController& Controller = AnimationSequenceBase->GetController();
		SetControllerCurveKeys(Controller, CurveName, Times, KeyData);
	}
}

bool UAnimationBlueprintLibrary::AddCurveInternal(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, int32 CurveFlags, ERawCurveTrackTypes SupportedCurveType)
{
	// Add or retrieve the smart name
	FAnimationCurveIdentifier CurveId(CurveName, SupportedCurveType);

	bool bCurveAdded = false;

	const FAnimCurveBase* ExistingCurve = AnimationSequenceBase->GetDataModel()->FindCurve(CurveId);
	if (ExistingCurve == nullptr)
	{
		IAnimationDataController& Controller = AnimationSequenceBase->GetController();
		Controller.AddCurve(CurveId, CurveFlags);
		bCurveAdded = AnimationSequenceBase->GetDataModel()->FindCurve(CurveId) != nullptr;	
	}
	else
	{
		// Curve already exists
	}

	return bCurveAdded;
}

bool UAnimationBlueprintLibrary::RemoveCurveInternal(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, ERawCurveTrackTypes SupportedCurveType)
{
	checkf(AnimationSequenceBase != nullptr, TEXT("Invalid Animation Sequence ptr"));
	bool bRemoved = false;
	
	IAnimationDataController& Controller = AnimationSequenceBase->GetController();

	const FAnimationCurveIdentifier CurveId(CurveName, SupportedCurveType);
	bRemoved = Controller.RemoveCurve(CurveId);

	return bRemoved;
}

void UAnimationBlueprintLibrary::DoesBoneNameExist(UAnimSequence* AnimationSequence, FName BoneName, bool& bExists)
{
	bExists = false;
	if (AnimationSequence)
	{
		USkeleton* Skeleton = AnimationSequence->GetSkeleton();
		if (Skeleton)
		{
			bExists = DoesBoneNameExistInternal(Skeleton, BoneName);
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "No Skeleton found for Animation Sequence %ls", *AnimationSequence->GetName());
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for DoesBoneNameExist");
	}
}

bool UAnimationBlueprintLibrary::DoesBoneNameExistInternal(USkeleton* Skeleton, FName BoneName)
{
	checkf(Skeleton != nullptr, TEXT("Invalid Skeleton ptr"));
	return Skeleton->GetReferenceSkeleton().FindBoneIndex(BoneName) != INDEX_NONE;
}

int32 UAnimationBlueprintLibrary::GetNumCurveMetaData(const USkeleton* Skeleton)
{
	if (Skeleton)
	{
		return Skeleton->GetNumCurveMetaData();
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Skeleton ptr for GetNumCurveMetaData");
	}

	return 0;
}

bool UAnimationBlueprintLibrary::AddCurveMetaData(USkeleton* Skeleton, FName CurveName, bool bTransact /*= true*/)
{
	if (Skeleton)
	{
		return Skeleton->AddCurveMetaData(CurveName, bTransact);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Skeleton ptr for AddCurveMetaData");
	}

	return false;
}

void UAnimationBlueprintLibrary::GetCurveMetaDataNames(const USkeleton* Skeleton, TArray<FName>& OutNames)
{
	if (Skeleton)
	{
		Skeleton->GetCurveMetaDataNames(OutNames);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Skeleton ptr for GetCurveMetaDataNames");
	}
}

bool UAnimationBlueprintLibrary::RenameCurveMetaData(USkeleton* Skeleton, FName OldName, FName NewName)
{
	if (Skeleton)
	{
		return Skeleton->RenameCurveMetaData(OldName, NewName);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Skeleton ptr for RenameCurveMetaData");
	}

	return false;
}

bool UAnimationBlueprintLibrary::RemoveCurveMetaData(USkeleton* Skeleton, FName CurveName)
{
	if (Skeleton)
	{
		return Skeleton->RemoveCurveMetaData(CurveName);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Skeleton ptr for RemoveCurveMetaData");
	}

	return false;
}

bool UAnimationBlueprintLibrary::GetCurveMetaDataMaterial(const USkeleton* Skeleton, FName CurveName)
{
	if (Skeleton)
	{
		return Skeleton->GetCurveMetaDataMaterial(CurveName);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Skeleton ptr for GetCurveMetaDataMaterial");
	}

	return false;
}

bool UAnimationBlueprintLibrary::GetCurveMetaDataMorphTarget(const USkeleton* Skeleton, FName CurveName)
{
	if (Skeleton)
	{
		return Skeleton->GetCurveMetaDataMorphTarget(CurveName);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Skeleton ptr for GetCurveMetaDataMorphTarget");
	}

	return false;
}

void UAnimationBlueprintLibrary::SetCurveMetaDataMaterial(USkeleton* Skeleton, FName CurveName, bool bOverrideMaterial)
{
	if (Skeleton)
	{
		Skeleton->SetCurveMetaDataMaterial(CurveName, bOverrideMaterial);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Skeleton ptr for SetCurveMetaDataMaterial");
	}
}


void UAnimationBlueprintLibrary::SetCurveMetaDataMorphTarget(USkeleton* Skeleton, FName CurveName, bool bOverrideMorphTarget)
{
	if (Skeleton)
	{
		Skeleton->SetCurveMetaDataMorphTarget(CurveName, bOverrideMorphTarget);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Skeleton ptr for SetCurveMetaDataMorphTarget");
	}
}

void UAnimationBlueprintLibrary::GetFloatKeys(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, TArray<float>& Times, TArray<float>& Values)
{
	if (AnimationSequenceBase)
	{
		GetCurveKeysInternal<float, FFloatCurve, ERawCurveTrackTypes::RCT_Float>(AnimationSequenceBase, CurveName, Times, Values);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for GetFloatKeys");
	}
}

void UAnimationBlueprintLibrary::GetVectorKeys(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, TArray<float>& Times, TArray<FVector>& Values)
{
	if (AnimationSequenceBase)
	{
		GetCurveKeysInternal<FVector, FVectorCurve, ERawCurveTrackTypes::RCT_Vector>(AnimationSequenceBase, CurveName, Times, Values);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for GetVectorKeys");
	}
}

void UAnimationBlueprintLibrary::GetTransformationKeys(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, TArray<float>& Times, TArray<FTransform>& Values)
{
	if (AnimationSequenceBase)
	{
		GetCurveKeysInternal<FTransform, FTransformCurve, ERawCurveTrackTypes::RCT_Transform>(AnimationSequenceBase, CurveName, Times, Values);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for GetTransformationKeys");
	}
}

float UAnimationBlueprintLibrary::GetFloatValueAtTime(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, float Time)
{
	if (AnimationSequenceBase)
	{
	    const FAnimationCurveIdentifier CurveId(CurveName, ERawCurveTrackTypes::RCT_Float);
	    const FFloatCurve* Curve = AnimationSequenceBase->GetDataModel()->FindFloatCurve(CurveId);
	    if (Curve)
	    {
		    return Curve->Evaluate(Time);
	    }
    
	    UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Float Curve Name for given Animation Sequence");
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for GetFloatValueAtTime");
	}

	return 0.0f;
}

template <typename DataType, typename CurveClass, ERawCurveTrackTypes CurveType>
void UAnimationBlueprintLibrary::GetCurveKeysInternal(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, TArray<float>& Times, TArray<DataType>& KeyData)
{
	checkf(AnimationSequenceBase != nullptr, TEXT("Invalid Animation Sequence ptr"));
	
	// Retrieve the curve by name
	const FAnimationCurveIdentifier CurveId(CurveName, CurveType);
	const CurveClass* Curve = static_cast<const CurveClass*>(AnimationSequenceBase->GetDataModel()->FindCurve(CurveId));

	if (Curve)
	{
		Curve->GetKeys(Times, KeyData);
		checkf(Times.Num() == KeyData.Num(), TEXT("Invalid key data retrieved from curve"));
	}
}

bool UAnimationBlueprintLibrary::DoesCurveExist(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, ERawCurveTrackTypes CurveType)
{
	bool bExistingCurve = false;

	if (AnimationSequenceBase)
	{
		FAnimationCurveIdentifier CurveId(CurveName, CurveType);
		const FAnimCurveBase* Curve = AnimationSequenceBase->GetDataModel()->FindCurve(CurveId);
		bExistingCurve = Curve != nullptr;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for DoesCurveExist");
	}

	return bExistingCurve;
}

ERawCurveTrackTypes UAnimationBlueprintLibrary::RetrieveCurveTypeForCurve(const UAnimSequenceBase* AnimationSequenceBase, FName CurveName)
{
	if(AnimationSequenceBase->GetDataModel()->FindCurve(FAnimationCurveIdentifier(CurveName, ERawCurveTrackTypes::RCT_Float)))
	{
		return ERawCurveTrackTypes::RCT_Float;
	}
	
	if(AnimationSequenceBase->GetDataModel()->FindCurve(FAnimationCurveIdentifier(CurveName, ERawCurveTrackTypes::RCT_Transform)))
	{
		return ERawCurveTrackTypes::RCT_Transform;
	}

	return ERawCurveTrackTypes::RCT_MAX;
}

void UAnimationBlueprintLibrary::AddMetaData(UAnimationAsset* AnimationAsset, TSubclassOf<UAnimMetaData> MetaDataClass, UAnimMetaData*& MetaDataInstance)
{
	if (AnimationAsset)
	{
		MetaDataInstance = NewObject<UAnimMetaData>(AnimationAsset, MetaDataClass, NAME_None, RF_Transactional);
		if (MetaDataInstance)
		{
			AnimationAsset->AddMetaData(MetaDataInstance);
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Failed to create instance for %ls", *MetaDataClass->GetName());
		}
		
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for AddMetaData");
	}
}

void UAnimationBlueprintLibrary::AddMetaDataObject(UAnimationAsset* AnimationAsset, UAnimMetaData* MetaDataObject)
{
	if (AnimationAsset && MetaDataObject)
	{
		if (MetaDataObject->GetOuter() == AnimationAsset)
		{
			AnimationAsset->AddMetaData(MetaDataObject);
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Outer for MetaData Instance %ls is not Animation Sequence %ls", *MetaDataObject->GetName(), *AnimationAsset->GetName());
		}		
	}
	else 
	{
		if (!AnimationAsset)
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for AddMetaDataObject");
		}
		
		if (!MetaDataObject)
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid MetaDataObject for AddMetaDataObject");
		}
	}
}

void UAnimationBlueprintLibrary::RemoveAllMetaData(UAnimationAsset* AnimationAsset)
{
	if (AnimationAsset)
	{
		AnimationAsset->EmptyMetaData();
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for RemoveAllMetaData");
	}
}

void UAnimationBlueprintLibrary::RemoveMetaData(UAnimationAsset* AnimationAsset, UAnimMetaData* MetaDataObject)
{
	if (AnimationAsset)
	{
		AnimationAsset->RemoveMetaData(MetaDataObject);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for RemoveMetaData");
	}
}

void UAnimationBlueprintLibrary::RemoveMetaDataOfClass(UAnimationAsset* AnimationAsset, TSubclassOf<UAnimMetaData> MetaDataClass)
{
	if (AnimationAsset)
	{
		TArray<UAnimMetaData*> MetaDataOfClass;
		GetMetaDataOfClass(AnimationAsset, MetaDataClass, MetaDataOfClass);
		AnimationAsset->RemoveMetaData(MetaDataOfClass);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for RemoveMetaDataOfClass");
	}	
}

void UAnimationBlueprintLibrary::GetMetaData(const UAnimationAsset* AnimationAsset, TArray<UAnimMetaData*>& MetaData)
{
	MetaData.Empty();
	if (AnimationAsset)
	{
		MetaData = AnimationAsset->GetMetaData();
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for GetMetaData");
	}
}

void UAnimationBlueprintLibrary::GetMetaDataOfClass(const UAnimationAsset* AnimationAsset, TSubclassOf<UAnimMetaData> MetaDataClass, TArray<UAnimMetaData*>& MetaDataOfClass)
{
	MetaDataOfClass.Empty();
	if (AnimationAsset)
	{
		const TArray<UAnimMetaData*>& MetaData = AnimationAsset->GetMetaData();
		for (UAnimMetaData* MetaDataInstance : MetaData)
		{
			if (MetaDataInstance->GetClass() == *MetaDataClass)
			{
				MetaDataOfClass.Add(MetaDataInstance);
			}
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for GetMetaDataOfClass");
	}
}

bool UAnimationBlueprintLibrary::ContainsMetaDataOfClass(const UAnimationAsset* AnimationAsset, TSubclassOf<UAnimMetaData> MetaDataClass)
{
	bool bContainsMetaData = false;
	if (AnimationAsset)
	{
		TArray<UAnimMetaData*> MetaData;
		GetMetaData(AnimationAsset, MetaData);
		bContainsMetaData = MetaData.FindByPredicate(
			[&](UAnimMetaData* MetaDataObject)
		{
			return (MetaDataObject->GetClass() == *MetaDataClass);
		}) != nullptr;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence for ContainsMetaDataOfClass");
	}

	return bContainsMetaData;	
}

void UAnimationBlueprintLibrary::AddVirtualBone(const UAnimSequence* AnimationSequence, FName SourceBoneName, FName TargetBoneName, FName& VirtualBoneName)
{
	if (AnimationSequence)
	{
		USkeleton* Skeleton = AnimationSequence->GetSkeleton();
		if (Skeleton)
		{
			const bool bSourceBoneExists = DoesBoneNameExistInternal(Skeleton, SourceBoneName);
			const bool bTargetBoneExists = DoesBoneNameExistInternal(Skeleton, TargetBoneName);
			const bool bVirtualBoneDoesNotExist = !DoesVirtualBoneNameExistInternal(Skeleton, VirtualBoneName);
			
			if (bSourceBoneExists && bTargetBoneExists && bVirtualBoneDoesNotExist)
			{
				const bool bAdded = Skeleton->AddNewVirtualBone(SourceBoneName, TargetBoneName, VirtualBoneName);
				if (bAdded)
				{
					Skeleton->HandleSkeletonHierarchyChange();
				}
				else
				{
					UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Virtual bone between %ls and %ls already exists on Skeleton %ls", *SourceBoneName.ToString(), *TargetBoneName.ToString(), *Skeleton->GetName());
				}
			}
			else
			{
				if (!bSourceBoneExists)
				{
					UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Bone Name %ls does not exist on Skeleton %ls", *SourceBoneName.ToString(), *Skeleton->GetName());
				}

				if (!bTargetBoneExists)
				{
					UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Bone Name %ls does not exist on Skeleton %ls", *TargetBoneName.ToString(), *Skeleton->GetName());
				}			
			}		
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "No Skeleton found for Animation Sequence %ls", *AnimationSequence->GetName());
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for AddVirtualBone");
	}
}

void UAnimationBlueprintLibrary::RemoveVirtualBone(const UAnimSequence* AnimationSequence, FName VirtualBoneName)
{
	if (AnimationSequence)
	{
		USkeleton* Skeleton = AnimationSequence->GetSkeleton();
		if (Skeleton)
		{
			if (DoesVirtualBoneNameExistInternal(Skeleton, VirtualBoneName))
			{
				TArray<FName> BoneNameArray;
				BoneNameArray.Add(VirtualBoneName);
				Skeleton->RemoveVirtualBones(BoneNameArray);
				Skeleton->HandleSkeletonHierarchyChange();
			}
			else
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Virtual Bone Name %ls already exists on Skeleton %ls", *VirtualBoneName.ToString(), *Skeleton->GetName());
			}
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "No Skeleton found for Animation Sequence %ls", *AnimationSequence->GetName());
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for RemoveVirtualBone");
	}
}

void UAnimationBlueprintLibrary::RemoveVirtualBones(const UAnimSequence* AnimationSequence, TArray<FName> VirtualBoneNames)
{
	if (AnimationSequence)
	{
		USkeleton* Skeleton = AnimationSequence->GetSkeleton();
		if (Skeleton)
		{
			for (FName& VirtualBoneName : VirtualBoneNames)
			{
				if (!DoesVirtualBoneNameExistInternal(Skeleton, VirtualBoneName))
				{
					UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Virtual Bone Name %ls already exists on Skeleton %ls", *VirtualBoneName.ToString(), *Skeleton->GetName());
				}
			}			

			Skeleton->RemoveVirtualBones(VirtualBoneNames);
			Skeleton->HandleSkeletonHierarchyChange();
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "No Skeleton found for Animation Sequence %ls", *AnimationSequence->GetName());
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for RemoveVirtualBones");
	}
}

void UAnimationBlueprintLibrary::RemoveAllVirtualBones(const UAnimSequence* AnimationSequence)
{
	if (AnimationSequence)
	{
		USkeleton* Skeleton = AnimationSequence->GetSkeleton();
		if (Skeleton)
		{
			TArray<FName> VirtualBoneNames;
			for (const FVirtualBone& VirtualBone : Skeleton->VirtualBones)
			{
				VirtualBoneNames.Add(VirtualBone.VirtualBoneName);
			}

			RemoveVirtualBones(AnimationSequence, VirtualBoneNames);
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "No Skeleton found for Animation Sequence %ls", *AnimationSequence->GetName());
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for RemoveAllVirtualBones");
	}
}

bool UAnimationBlueprintLibrary::DoesVirtualBoneNameExistInternal(USkeleton* Skeleton, FName BoneName)
{
	checkf(Skeleton != nullptr, TEXT("Invalid Skeleton ptr"));
	return Skeleton->VirtualBones.IndexOfByPredicate([&](const FVirtualBone& VirtualBone) { return VirtualBone.VirtualBoneName == BoneName; }) != INDEX_NONE;
}

void UAnimationBlueprintLibrary::GetSequenceLength(const UAnimSequenceBase* AnimationSequenceBase, float& Length)
{
	Length = 0.0f;
	if (AnimationSequenceBase)
	{
		Length = AnimationSequenceBase->GetPlayLength();
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for GetSequenceLength");
	}
}

void UAnimationBlueprintLibrary::GetRateScale(const UAnimSequenceBase* AnimationSequenceBase, float& RateScale)
{
	RateScale = 0.0f;
	if (AnimationSequenceBase)
	{
		RateScale = AnimationSequenceBase->RateScale;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for GetRateScale");
	}
}

void UAnimationBlueprintLibrary::SetRateScale(UAnimSequenceBase* AnimationSequenceBase, float RateScale)
{	
	if (AnimationSequenceBase)
	{
		AnimationSequenceBase->RateScale = RateScale;
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for SetRateScale");
	}
}

void UAnimationBlueprintLibrary::GetFrameAtTime(const UAnimSequenceBase* AnimationSequenceBase, const float Time, int32& Frame)
{
	Frame = 0;
	if (AnimationSequenceBase)
	{
		Frame = AnimationSequenceBase->GetFrameAtTime(Time);
	}
	else
	{		
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for GetFrameAtTime");
	}
}

void UAnimationBlueprintLibrary::GetTimeAtFrame(const UAnimSequenceBase* AnimationSequenceBase, const int32 Frame, float& Time)
{
	Time = 0.0f;
	if (AnimationSequenceBase)
	{
		Time = GetTimeAtFrameInternal(AnimationSequenceBase, Frame);
	}
	else
	{		
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for GetTimeAtFrame");
	}
}

float UAnimationBlueprintLibrary::GetTimeAtFrameInternal(const UAnimSequenceBase* AnimationSequenceBase, const int32 Frame)
{
	return AnimationSequenceBase->GetTimeAtFrame(Frame);
}

void UAnimationBlueprintLibrary::IsValidTime(const UAnimSequenceBase* AnimationSequenceBase, const float Time, bool& IsValid)
{
	IsValid = false;
	if (AnimationSequenceBase)
	{
		IsValid = IsValidTimeInternal(AnimationSequenceBase, Time);
	}
	else
	{		
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for IsValidTime");
	}
}

bool UAnimationBlueprintLibrary::IsValidTimeInternal(const UAnimSequenceBase* AnimationSequenceBase, const float Time)
{
	return FMath::IsWithinInclusive(Time, 0.0f, AnimationSequenceBase->GetPlayLength());
}

namespace UE::AnimationBlueprintLibrary::Private
{
	struct FTimecodeBoneAttributeNames
	{
		FTimecodeBoneAttributeNames()
		{
			TCHourAttrName = TEXT("TCHour");
			TCMinuteAttrName = TEXT("TCMinute");
			TCSecondAttrName = TEXT("TCSecond");
			TCFrameAttrName = TEXT("TCFrame");
			TCSubframeAttrName = TEXT("TCSubframe");
			TCRateAttrName = TEXT("TCRate");
			TCSlateAttrName = TEXT("TCSlate");
			if (const UAnimationSettings* AnimationSettings = UAnimationSettings::Get())
			{
				TCHourAttrName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.HourAttributeName;
				TCMinuteAttrName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.MinuteAttributeName;
				TCSecondAttrName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.SecondAttributeName;
				TCFrameAttrName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.FrameAttributeName;
				TCSubframeAttrName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.SubframeAttributeName;
				TCRateAttrName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.RateAttributeName;
				TCSlateAttrName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.TakenameAttributeName;
			}
		}

		bool Contains(const FName AttributeName) const
		{
			return AttributeName == TCHourAttrName
				|| AttributeName == TCMinuteAttrName
				|| AttributeName == TCSecondAttrName
				|| AttributeName == TCFrameAttrName
				|| AttributeName == TCSubframeAttrName
				|| AttributeName == TCRateAttrName
				|| AttributeName == TCSlateAttrName;
		}

		FName TCHourAttrName;
		FName TCMinuteAttrName;
		FName TCSecondAttrName;
		FName TCFrameAttrName;
		FName TCSubframeAttrName;
		FName TCRateAttrName;
		FName TCSlateAttrName;
	};
}

FName UAnimationBlueprintLibrary::FindBoneNameWithTimecodeAttributes(const UAnimSequenceBase* AnimationSequenceBase)
{
	if (!AnimationSequenceBase || !AnimationSequenceBase->GetSkeleton())
	{
		return NAME_None;
	}

	const IAnimationDataModel* AnimDataModel = AnimationSequenceBase->GetDataModel();
	if (!AnimDataModel || !AnimDataModel->HasBeenPopulated())
	{
		return NAME_None;
	}

	using namespace UE::AnimationBlueprintLibrary::Private;
	FTimecodeBoneAttributeNames TimecodeBoneAttributeNames;

	const TArray<FMeshBoneInfo>& BoneInfos = AnimationSequenceBase->GetSkeleton()->GetReferenceSkeleton().GetRefBoneInfo();
	for (FMeshBoneInfo BoneInfo : BoneInfos)
	{
		if (!AnimDataModel->IsValidBoneTrackName(BoneInfo.Name))
		{
			continue;
		}
		TArray<const FAnimatedBoneAttribute*> BoneAttributes;
		AnimDataModel->GetAttributesForBone(BoneInfo.Name, BoneAttributes);
		for (const FAnimatedBoneAttribute* BoneAttribute : BoneAttributes)
		{
			if (!BoneAttribute)
			{
				continue;
			}
			const FName& BoneAttributeName = BoneAttribute->Identifier.GetName();

			if (TimecodeBoneAttributeNames.Contains(BoneAttributeName))
			{
				return BoneInfo.Name;
			}
		}
	}
	return NAME_None;
}

bool UAnimationBlueprintLibrary::EvaluateRootBoneTimecodeAttributesAtTime(const UAnimSequenceBase* AnimationSequenceBase, const float EvalTime, FQualifiedFrameTime& OutQualifiedFrameTime)
{
	if (!AnimationSequenceBase || !AnimationSequenceBase->GetSkeleton())
	{
		return false;
	}

	const IAnimationDataModel* AnimDataModel = AnimationSequenceBase->GetDataModel();
	if (!AnimDataModel || !AnimDataModel->HasBeenPopulated())
	{
		return false;
	}

	FString OutSlate;
	const FName RootBoneName = AnimationSequenceBase->GetSkeleton()->GetReferenceSkeleton().GetBoneName(0);
	return EvaluateBoneTimecodeAndSlateAttributesAtTime(RootBoneName, AnimationSequenceBase, EvalTime, OutQualifiedFrameTime, OutSlate);
}

bool UAnimationBlueprintLibrary::EvaluateBoneTimecodeAndSlateAttributesAtTime(const FName BoneName, const UAnimSequenceBase* AnimationSequenceBase, const float EvalTime, FQualifiedFrameTime& OutQualifiedFrameTime, FString& OutSlate)
{
	if (!AnimationSequenceBase || !AnimationSequenceBase->GetSkeleton())
	{
		return false;
	}

	const IAnimationDataModel* AnimDataModel = AnimationSequenceBase->GetDataModel();
	if (!AnimDataModel || !AnimDataModel->HasBeenPopulated())
	{
		return false;
	}

	if (!AnimDataModel->IsValidBoneTrackName(BoneName))
	{
		return false;
	}

	TArray<const FAnimatedBoneAttribute*> RootBoneAttributes;
	AnimDataModel->GetAttributesForBone(BoneName, RootBoneAttributes);

	using namespace UE::AnimationBlueprintLibrary::Private;
	FTimecodeBoneAttributeNames TimecodeBoneAttributeNames;

	bool bHasTimecodeBoneAttributes = false;

	FTimecode Timecode;
	float SubFrame = 0.0f;

	FString TimecodeRateAsString;
	double TimecodeRateAsDecimal = 0.0;

	for (const FAnimatedBoneAttribute* RootBoneAttribute : RootBoneAttributes)
	{
		if (!RootBoneAttribute)
		{
			continue;
		}

		const FName& BoneAttributeName = RootBoneAttribute->Identifier.GetName();

		// Avoid evaluating non-timecode bone attributes.
		if (!TimecodeBoneAttributeNames.Contains(BoneAttributeName))
		{
			continue;
		}

		if (!RootBoneAttribute->Curve.CanEvaluate())
		{
			continue;
		}

		float FloatValue = 0.0f;
		int32 IntValue = 0;
		FString StringValue;

		// Support timecode attribute curves that are either float-typed or integer-typed.
		if (RootBoneAttribute->Curve.GetScriptStruct() == FFloatAnimationAttribute::StaticStruct())
		{
			const FFloatAnimationAttribute EvaluatedAttribute = RootBoneAttribute->Curve.Evaluate<FFloatAnimationAttribute>(EvalTime);
			FloatValue = EvaluatedAttribute.Value;
			IntValue = static_cast<int32>(FloatValue);
		}
		else if (RootBoneAttribute->Curve.GetScriptStruct() == FIntegerAnimationAttribute::StaticStruct())
		{
			const FIntegerAnimationAttribute EvaluatedAttribute = RootBoneAttribute->Curve.Evaluate<FIntegerAnimationAttribute>(EvalTime);
			IntValue = EvaluatedAttribute.Value;
			FloatValue = static_cast<float>(IntValue);
		}
		else if (RootBoneAttribute->Curve.GetScriptStruct() == FStringAnimationAttribute::StaticStruct())
		{
			const FStringAnimationAttribute EvaluatedAttribute = RootBoneAttribute->Curve.Evaluate<FStringAnimationAttribute>(EvalTime);
			StringValue = EvaluatedAttribute.Value;
			if (StringValue.IsNumeric())
			{
				LexFromString(FloatValue, *StringValue);
				IntValue = static_cast<int32>(FloatValue);
			}
		}
		else
		{
			continue;
		}

		if (BoneAttributeName.IsEqual(TimecodeBoneAttributeNames.TCHourAttrName))
		{
			Timecode.Hours = IntValue;
			bHasTimecodeBoneAttributes = true;
		}
		else if (BoneAttributeName.IsEqual(TimecodeBoneAttributeNames.TCMinuteAttrName))
		{
			Timecode.Minutes = IntValue;
			bHasTimecodeBoneAttributes = true;
		}
		else if (BoneAttributeName.IsEqual(TimecodeBoneAttributeNames.TCSecondAttrName))
		{
			Timecode.Seconds = IntValue;
			bHasTimecodeBoneAttributes = true;
		}
		else if (BoneAttributeName.IsEqual(TimecodeBoneAttributeNames.TCFrameAttrName))
		{
			Timecode.Frames = IntValue;
			bHasTimecodeBoneAttributes = true;
		}
		else if (BoneAttributeName.IsEqual(TimecodeBoneAttributeNames.TCSubframeAttrName))
		{
			SubFrame = FloatValue;
			bHasTimecodeBoneAttributes = true;
		}
		else if (BoneAttributeName.IsEqual(TimecodeBoneAttributeNames.TCRateAttrName))
		{
			TimecodeRateAsString = StringValue;
			TimecodeRateAsDecimal = FloatValue;
			// Don't consider this attribute when determining whether timecode value attributes are
			// present, since it can't be useful on its own.
		}
		else if (BoneAttributeName.IsEqual(TimecodeBoneAttributeNames.TCSlateAttrName))
		{
			OutSlate = StringValue;
		}
	}

	if (!bHasTimecodeBoneAttributes)
	{
		return false;
	}

	// We'll fall back on the sampling frame rate if we can't determine the
	// original source frame rate.
	FFrameRate FrameRate = AnimationSequenceBase->GetSamplingFrameRate();

	if (!TimecodeRateAsString.IsEmpty() && TryParseString(FrameRate, *TimecodeRateAsString))
	{
		Timecode.bDropFrameFormat = FTimecode::IsValidDropFormatTimecodeRate(TimecodeRateAsString);
	}
	else if (TimecodeRateAsDecimal > 1.0)
	{
		if (const FCommonFrameRateInfo* TimecodeRateInfo = FCommonFrameRates::Find(TimecodeRateAsDecimal))
		{
			FrameRate = TimecodeRateInfo->FrameRate;
		}
		else
		{
			FrameRate = FFrameRate(static_cast<int32>(FMath::TruncToInt(TimecodeRateAsDecimal)), 1);
		}
	}
	else if (const UAnimSequence* AnimSequence = AnimDataModel->GetAnimationSequence())
	{
		if (AnimSequence->ImportFileFramerate > 0.0f)
		{
			FrameRate = FFrameRate(static_cast<int32>(AnimSequence->ImportFileFramerate), 1);
		}
	}

	// Some pipelines may author subframe values that are compatible with the
	// engine's notion of a subframe, which is a floating point value between
	// zero and one representing a percentage of a frame. Others may author
	// subframe as integer values from zero to N instead, incrementing by one
	// each subframe until the next frame is reached.
	// Since this data is user-supplied, we don't want to trip over the
	// checkSlow() in FFrameTime's constructor, so we apply the same clamping
	// it does to bring the value into range here.
	// Clients that are interested in the exact subframe value that was
	// authored should query it using EvaluateRootBoneTimecodeSubframeAttributeAtTime().
	SubFrame = FMath::Clamp(SubFrame + 0.5f - 0.5f, 0.f, FFrameTime::MaxSubframe);

	OutQualifiedFrameTime = FQualifiedFrameTime(
		FFrameTime(Timecode.ToFrameNumber(FrameRate), SubFrame),
		FrameRate);

	return true;
}

bool UAnimationBlueprintLibrary::EvaluateRootBoneTimecodeSubframeAttributeAtTime(const UAnimSequenceBase* AnimationSequenceBase, const float EvalTime, float& OutSubframe)
{
	if (!AnimationSequenceBase || !AnimationSequenceBase->GetSkeleton())
	{
		return false;
	}

	const IAnimationDataModel* AnimDataModel = AnimationSequenceBase->GetDataModel();
	if (!AnimDataModel)
	{
		return false;
	}

	const FName RootBoneName = AnimationSequenceBase->GetSkeleton()->GetReferenceSkeleton().GetBoneName(0);
	if (!AnimDataModel->IsValidBoneTrackName(RootBoneName))
	{
		return false;
	}

	FName TCSubframeAttrName(TEXT("TCSubframe"));
	if (const UAnimationSettings* AnimationSettings = UAnimationSettings::Get())
	{
		TCSubframeAttrName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.SubframeAttributeName;
	}

	FAnimationAttributeIdentifier SubframeAttributeIdentifier(TCSubframeAttrName, 0, RootBoneName, FFloatAnimationAttribute::StaticStruct());
	const FAnimatedBoneAttribute* RootBoneSubframeAttribute = AnimDataModel->FindAttribute(SubframeAttributeIdentifier);
	if (!RootBoneSubframeAttribute)
	{
		return false;
	}

	if (!RootBoneSubframeAttribute->Curve.CanEvaluate())
	{
		return false;
	}

	const FFloatAnimationAttribute EvaluatedAttribute = RootBoneSubframeAttribute->Curve.Evaluate<FFloatAnimationAttribute>(EvalTime);
	OutSubframe = EvaluatedAttribute.Value;

	return true;
}

void UAnimationBlueprintLibrary::FindBonePathToRoot(const UAnimSequenceBase* AnimationSequenceBase, FName BoneName, TArray<FName>& BonePath)
{
	BonePath.Empty();
	if (AnimationSequenceBase)
	{
		BonePath.Add(BoneName);
		int32 BoneIndex = AnimationSequenceBase->GetSkeleton()->GetReferenceSkeleton().FindRawBoneIndex(BoneName);		
		if (BoneIndex != INDEX_NONE)
		{
			while (BoneIndex != INDEX_NONE)
			{
				const int32 ParentBoneIndex = AnimationSequenceBase->GetSkeleton()->GetReferenceSkeleton().GetRawParentIndex(BoneIndex);
				if (ParentBoneIndex != INDEX_NONE)
				{
					BonePath.Add(AnimationSequenceBase->GetSkeleton()->GetReferenceSkeleton().GetBoneName(ParentBoneIndex));
				}

				BoneIndex = ParentBoneIndex;
			}
		}
		else
		{
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Bone name %ls not found in Skeleton %ls", *BoneName.ToString(), *AnimationSequenceBase->GetSkeleton()->GetName());
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for FindBonePathToRoot");
	}
}

void UAnimationBlueprintLibrary::RemoveBoneAnimation(UAnimSequence* AnimationSequence, FName BoneName, bool bIncludeChildren /*= true*/, bool bFinalize /*= true*/)
{
	if (AnimationSequence)
	{
		TArray<FName> TrackNames;
		AnimationSequence->GetDataModel()->GetBoneTrackNames(TrackNames);
		
		if (TrackNames.Contains(BoneName))
		{
			TArray<FName> TracksToRemove;
			TracksToRemove.Add(BoneName);

			// remove all children if required
			if (bIncludeChildren)
			{
				USkeleton* Skeleton = AnimationSequence->GetSkeleton();
				if (Skeleton)
				{
					const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
					const int32 ParentBoneIndex = RefSkeleton.FindBoneIndex(BoneName);

					// slow
					for (const FName& TrackName : TrackNames)
					{
						if (TrackName != BoneName)
						{
							const int32 ChildBoneIndex = RefSkeleton.FindBoneIndex(TrackName);
							if (RefSkeleton.BoneIsChildOf(ChildBoneIndex, ParentBoneIndex))
							{
								TracksToRemove.Add(TrackName);
							}
						}
					}
				}
			}

			IAnimationDataController& Controller = AnimationSequence->GetController();

			IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("RemoveBoneAnimation_Bracket", "Removing Bone Animation Track"));
			for (const FName& TrackName : TracksToRemove)
			{
				Controller.RemoveBoneTrack(TrackName);
			}
		}
		else
		{
			// print warning with track index
			UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Bone Name for the animation.");
		}
	}
}


void UAnimationBlueprintLibrary::RemoveBoneSelectiveAnimation(UAnimSequence* AnimationSequence, FName BoneName, TArray<FName> ChildrenExcluded, bool bIncludeChildren /*= true*/, bool bExcludeRecursively /*= false*/, bool bFinalize /*= true*/)
{
	if (AnimationSequence)
	{
		TArray<FName> TrackNames;
		AnimationSequence->GetDataModel()->GetBoneTrackNames(TrackNames);
		
		if (TrackNames.Contains(BoneName))
		{
			TArray<FName> TracksToRemove;
			TracksToRemove.Add(BoneName);


			// remove all children if required
			if (bIncludeChildren)
			{
				USkeleton* Skeleton = AnimationSequence->GetSkeleton();
				if (Skeleton)
				{
					const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
					const int32 ParentBoneIndex = RefSkeleton.FindBoneIndex(BoneName);
			
					TArray<FName> ChildrenExcludedDescendents;
					
					if (bExcludeRecursively)
					{
						// create full list of child exclude with descendent search
						for (const FName& ChildExcludedTrackName : ChildrenExcluded)
						{
							const int32 ChildExcludedIndex = RefSkeleton.FindBoneIndex(ChildExcludedTrackName);
							if (RefSkeleton.IsValidIndex(ChildExcludedIndex))
							{
								for (const FName& ChildCandidateTrackName : TrackNames)
								{
									const int32 ChildCandidateTrackIndex = RefSkeleton.FindBoneIndex(ChildCandidateTrackName);
									if (RefSkeleton.BoneIsChildOf(ChildCandidateTrackIndex, ChildExcludedIndex))
									{
										ChildrenExcludedDescendents.Add(ChildCandidateTrackName);
									}
								}
							}
							else
							{
								// print info with track index
								UE_LOGF(LogAnimationBlueprintLibrary, Display, "Invalid Child Exclusion Bone:");
							}
						} 
					}
					
					// slow
					for (const FName& TrackName : TrackNames)
					{
						if (TrackName != BoneName && !ChildrenExcluded.Contains(TrackName) && !ChildrenExcludedDescendents.Contains(TrackName))
						{
							const int32 ChildBoneIndex = RefSkeleton.FindBoneIndex(TrackName);
							if (RefSkeleton.BoneIsChildOf(ChildBoneIndex, ParentBoneIndex))
							{
								TracksToRemove.Add(TrackName);
							}
						}
					}
				}
			}

			IAnimationDataController& Controller = AnimationSequence->GetController();

			IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("RemoveBoneAnimation_Bracket", "Removing Bone Animation Track"));
			for (const FName& TrackName : TracksToRemove)
			{
				Controller.RemoveBoneTrack(TrackName);
			}
		}
		else
		{
			// print info with track index
			UE_LOGF(LogAnimationBlueprintLibrary, Display, "Invalid Bone Name for the animation.");
		}
	}
}

void UAnimationBlueprintLibrary::RemoveAllBoneAnimation(UAnimSequence* AnimationSequence)
{
	if (AnimationSequence)
	{
		IAnimationDataController& Controller = AnimationSequence->GetController();
		IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("RemoveAllBoneAnimation_Bracket", "Removing all Bone Animation and Transform Curve Tracks"));
		Controller.RemoveAllBoneTracks();
		Controller.RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Transform);		
	}
}

static void RecursiveRetrieveAnimationGraphs(UEdGraph* EdGraph, TArray<UAnimationGraph*>& OutAnimationGraphs)
{
	if (UAnimationGraph* AnimGraph = Cast<UAnimationGraph>(EdGraph))
	{
		OutAnimationGraphs.Add(AnimGraph);
	}
	
	for (TObjectPtr<class UEdGraph>& SubGraph : EdGraph->SubGraphs)
	{		
		RecursiveRetrieveAnimationGraphs(SubGraph, OutAnimationGraphs);
	}
}

void UAnimationBlueprintLibrary::GetAnimationGraphs(UAnimBlueprint* AnimationBlueprint, TArray<UAnimationGraph*>& AnimationGraphs)
{
	if (AnimationBlueprint)
	{
		for (UEdGraph* EdGraph : AnimationBlueprint->FunctionGraphs)
		{
			RecursiveRetrieveAnimationGraphs(EdGraph, AnimationGraphs);
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Blueprint");
	}
}

void UAnimationBlueprintLibrary::GetNodesOfClass(UAnimBlueprint* AnimationBlueprint, TSubclassOf<UAnimGraphNode_Base> NodeClass, TArray<UAnimGraphNode_Base*>& GraphNodes, bool bIncludeChildClasses /*= true*/)
{
	TArray<UAnimationGraph*> AnimationGraphs;
	GetAnimationGraphs(AnimationBlueprint, AnimationGraphs);
	for (UAnimationGraph* AnimGraph : AnimationGraphs)
	{
		AnimGraph->GetGraphNodesOfClass(NodeClass, GraphNodes, bIncludeChildClasses);
	}
}

void UAnimationBlueprintLibrary::AddNodeAssetOverride(UAnimBlueprint* AnimBlueprint, const UAnimationAsset* Target, UAnimationAsset* Override, bool bPrintAppliedOverrides /*= false*/)
{
	if (AnimBlueprint)
	{
		if (Target && Override)
		{
			// Target and override animation asset types have to match
			if (Target->GetClass() == Override->GetClass())			
			{
				TArray<UBlueprint*> BlueprintHierarchy;
				AnimBlueprint->GetBlueprintHierarchyFromClass(AnimBlueprint->GetAnimBlueprintGeneratedClass(), BlueprintHierarchy);

				TArray<const UAnimGraphNode_Base*> OverridableNodes;

				// Search from 1 as 0 is this class and we're looking for it's parents
				for(int32 BlueprintIndex = 1; BlueprintIndex < BlueprintHierarchy.Num(); ++BlueprintIndex)
				{
					const UBlueprint* CurrentBlueprint = BlueprintHierarchy[BlueprintIndex];

					TArray<UEdGraph*> Graphs;
					CurrentBlueprint->GetAllGraphs(Graphs);

					for(const UEdGraph* Graph : Graphs)
					{
						for(const UEdGraphNode* Node : Graph->Nodes)
						{
							const UAnimGraphNode_Base* AnimNode = Cast<UAnimGraphNode_Base>(Node);
							// Find any overridable node with Target set as its current value
							if(AnimNode && AnimNode->GetAnimationAsset() == Target)
							{
								OverridableNodes.Add(AnimNode);
							}
						}
					}
				}

				// Apply overrides
				for (const UAnimGraphNode_Base* OverrideNode : OverridableNodes)
				{
					const FGuid NodeGUID = OverrideNode->NodeGuid;
					
					FAnimParentNodeAssetOverride* OverridePtr = AnimBlueprint->ParentAssetOverrides.FindByPredicate([NodeGUID](const FAnimParentNodeAssetOverride& Other)
					{
						return Other.ParentNodeGuid == NodeGUID;
					});
						
					if (OverridePtr == nullptr)
					{
						OverridePtr = &AnimBlueprint->ParentAssetOverrides.AddDefaulted_GetRef();
					}

					check(OverridePtr != nullptr);

					auto GetOverrideNodeTitle = [OverrideNode]() -> FString
					{
						if (const UAnimGraphNode_AssetPlayerBase * AssetPlayerBase = Cast<UAnimGraphNode_AssetPlayerBase>(OverrideNode))
						{
							return AssetPlayerBase->GetNodeTitle(ENodeTitleType::FullTitle).ToString();
						}

						return OverrideNode->GetName();
					};

					if (OverridePtr->NewAsset != Override)
					{
						// Setup override values
						OverridePtr->NewAsset = Override;
						OverridePtr->ParentNodeGuid = NodeGUID;
					
						AnimBlueprint->NotifyOverrideChange(*OverridePtr);

						if (bPrintAppliedOverrides)
						{
							UE_LOGF(LogAnimationBlueprintLibrary, Display, "Set Animation Blueprint asset override for %ls\n\tAnimation Node: %ls\n\tAnimation Node Bueprint: %ls\n\tOriginal: %ls\n\tOverride: %ls",
								*AnimBlueprint->GetPathName(),
								*GetOverrideNodeTitle(),
								*OverrideNode->GetAnimBlueprint()->GetPathName(),
								*Target->GetPathName(),
								*Override->GetPathName()							
							);
						}
					}
					else
					{
						
						UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Found matching pre-existing Animation Blueprint asset override for %ls\n\tAnimation Node: %ls\n\tAnimation Node Bueprint: %ls\n\tOriginal: %ls\n\tOverride: %ls",
								*AnimBlueprint->GetPathName(),
								*GetOverrideNodeTitle(),
								*OverrideNode->GetAnimBlueprint()->GetPathName(),
								*Target->GetPathName(),
								*Override->GetPathName()							
							);
					}
				}

				if (OverridableNodes.Num())
				{					
					FBlueprintEditorUtils::MarkBlueprintAsModified(AnimBlueprint);
				}
			}
			else
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Error, "Failed to add override as Target and Override class do not match, expected %ls but found %ls", *Target->GetClass()->GetName(), *Override->GetClass()->GetName());
			}
		}
		else
		{
			if (Target == nullptr)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Error, "Failed to add override as provided Target asset is invalid");
			}
			
			if (Override == nullptr)
			{
				UE_LOGF(LogAnimationBlueprintLibrary, Error, "Failed to add override as provided Override asset is invalid");
			}
		}
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Error, "Failed to add override as provided Animation Blueprint is invalid");
	}
}

void UAnimationBlueprintLibrary::GetBonePoseForTime(const UAnimSequenceBase* AnimationSequenceBase, FName BoneName, float Time, bool bExtractRootMotion, FTransform& Pose)
{
	Pose.SetIdentity();
	if (AnimationSequenceBase)
	{
		TArray<FName> BoneNameArray;
		TArray<FTransform> PoseArray;
		BoneNameArray.Add(BoneName);
		GetBonePosesForTimeInternal(AnimationSequenceBase, BoneNameArray, Time, bExtractRootMotion, PoseArray);
		Pose = PoseArray[0];
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for GetBonePoseForTime");
	}
}

void UAnimationBlueprintLibrary::GetBonePoseForFrame(const UAnimSequenceBase* AnimationSequenceBase, FName BoneName, int32 Frame, bool bExtractRootMotion, FTransform& Pose)
{
	Pose.SetIdentity();
	if (AnimationSequenceBase)
	{
		TArray<FName> BoneNameArray;
		TArray<FTransform> PoseArray;
		BoneNameArray.Add(BoneName);
		GetBonePosesForTimeInternal(AnimationSequenceBase, BoneNameArray, GetTimeAtFrameInternal(AnimationSequenceBase, Frame), bExtractRootMotion, PoseArray);
		Pose = PoseArray[0];
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for GetBonePoseForFrame");
	}
}

void UAnimationBlueprintLibrary::GetBonePosesForTime(const UAnimSequenceBase* AnimationSequenceBase, TArray<FName> BoneNames, float Time, bool bExtractRootMotion, TArray<FTransform>& Poses, const USkeletalMesh* PreviewMesh /*= nullptr*/)
{
	GetBonePosesForTimeInternal(AnimationSequenceBase, BoneNames, Time, bExtractRootMotion, Poses, PreviewMesh);
}

void UAnimationBlueprintLibrary::GetBonePosesForFrame(const UAnimSequenceBase* AnimationSequenceBase, TArray<FName> BoneNames, int32 Frame, bool bExtractRootMotion, TArray<FTransform>& Poses, const USkeletalMesh* PreviewMesh /*= nullptr*/)
{
	Poses.Empty(BoneNames.Num());
	if (AnimationSequenceBase)
	{
		GetBonePosesForTimeInternal(AnimationSequenceBase, BoneNames, GetTimeAtFrameInternal(AnimationSequenceBase, Frame), bExtractRootMotion, Poses, PreviewMesh);
	}
	else
	{
		UE_LOGF(LogAnimationBlueprintLibrary, Warning, "Invalid Animation Sequence supplied for GetBonePosesForFrame");
	}
}

template void UAnimationBlueprintLibrary::AddCurveKeysInternal<float, FFloatCurve, ERawCurveTrackTypes::RCT_Float>(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, const TArray<float>& Times, const TArray<float>& KeyData);
template void UAnimationBlueprintLibrary::AddCurveKeysInternal<FVector, FVectorCurve, ERawCurveTrackTypes::RCT_Vector>(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, const TArray<float>& Times, const TArray<FVector>& KeyData);
template void UAnimationBlueprintLibrary::AddCurveKeysInternal<FTransform, FTransformCurve, ERawCurveTrackTypes::RCT_Transform>(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, const TArray<float>& Times, const TArray<FTransform>& KeyData);

template void UAnimationBlueprintLibrary::GetCurveKeysInternal<float, FFloatCurve, ERawCurveTrackTypes::RCT_Float>(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, TArray<float>& Times, TArray<float>& KeyData);
template void UAnimationBlueprintLibrary::GetCurveKeysInternal<FVector, FVectorCurve, ERawCurveTrackTypes::RCT_Vector>(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, TArray<float>& Times, TArray<FVector>& KeyData);
template void UAnimationBlueprintLibrary::GetCurveKeysInternal<FTransform, FTransformCurve, ERawCurveTrackTypes::RCT_Transform>(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, TArray<float>& Times, TArray<FTransform>& KeyData);

void UAnimationBlueprintLibrary::SetSkeletonPreviewMesh(USkeleton* Skeleton, USkeletalMesh* PreviewMesh, bool bMarkAsDirty)
{
	if (Skeleton)
	{
		Skeleton->SetPreviewMesh(PreviewMesh, bMarkAsDirty);
	}
}

 USkeletalMesh* UAnimationBlueprintLibrary::GetSkeletonPreviewMesh(USkeleton* Skeleton, bool bFindIfNotSet)
 {
	if (Skeleton)
	{
		return Skeleton->GetPreviewMesh(bFindIfNotSet);
	}
	return nullptr;
 }

 void UAnimationBlueprintLibrary::SetSkeletonAdditionalPreviewMeshes(USkeleton* Skeleton, UDataAsset* AdditionalMeshes)
 {
	if (Skeleton)
	{
		Skeleton->SetAdditionalPreviewSkeletalMeshes(AdditionalMeshes);
	}
 }

 UDataAsset* UAnimationBlueprintLibrary::GetSkeletonAdditionalPreviewMeshes(const USkeleton* Skeleton)
 {
	if (Skeleton)
	{
		return Skeleton->GetAdditionalPreviewSkeletalMeshes();
	}
	return nullptr;
 }

#undef LOCTEXT_NAMESPACE // "AnimationBlueprintLibrary"
