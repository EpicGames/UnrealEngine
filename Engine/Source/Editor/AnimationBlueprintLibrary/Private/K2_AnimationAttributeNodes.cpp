// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2_AnimationAttributeNodes.h"

#include "Animation/AttributeTypes.h"
#include "AnimationAttributeBlueprintLibrary.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/MemberReference.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "StructUtils/UserDefinedStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2_AnimationAttributeNodes)

#define LOCTEXT_NAMESPACE "K2Node_AnimationAttributeNodes"

FName UK2Node_BaseAttributeActionNode::ValuePinName(TEXT("Value"));
FName UK2Node_BaseAttributeActionNode::ValuesPinName(TEXT("Values"));

FText UK2Node_BaseAttributeActionNode::GetNodeTitle(ENodeTitleType::Type Title) const
{
	ensure(AttributeValuePinName != NAME_None);
	ensure(!AttributeActionFormat.IsEmpty());
	
	UEdGraphPin* Pin = FindPin(AttributeValuePinName);
	const UScriptStruct* AttributeTypeStruct = Pin ? Cast<UScriptStruct>(Pin->PinType.PinSubCategoryObject.Get()) : nullptr;
	return FText::Format(AttributeActionFormat, AttributeTypeStruct ? FText::FromString(AttributeTypeStruct->GetName()) : LOCTEXT("AttributeSub", "Attribute"));	
}

void UK2Node_BaseAttributeActionNode::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	ensure(AttributeValuePinName != NAME_None);
	
	const UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

bool UK2Node_BaseAttributeActionNode::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	ensure(AttributeValuePinName != NAME_None);
	
	UEdGraphPin* Pin = FindPinChecked(AttributeValuePinName);
	if (Pin && MyPin == Pin)
	{
		bool bAllowed = false;
		// Ensure the pin is a struct, and in an array pin in case it is expected/required to be
		if (OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
		{
			if (AttributeValuePinName == ValuesPinName && !OtherPin->PinType.IsArray())
			{
				OutReason = LOCTEXT("NotArrayTypePin", "Must be an array of registered Animation Attributes.").ToString();
			}
			else if (AttributeValuePinName == ValuePinName && OtherPin->PinType.IsContainer())
			{
				OutReason = LOCTEXT("ContainerTypePin","Must not be a container pin type.").ToString();
			}
			else if (const UScriptStruct* AttributeType = Cast<UScriptStruct>(OtherPin->PinType.PinSubCategoryObject.Get()))
			{
				bAllowed = UE::Anim::AttributeTypes::IsTypeRegistered(AttributeType);

				if (!bAllowed)
				{
					OutReason = FText::Format(LOCTEXT("UnregisterdStructureTypeFormat", "Must be a registered Animation Attribute, which {0} is not."), FText::FromName(AttributeType->GetFName())).ToString();
				}
				else if (AttributeType->IsA<UUserDefinedStruct>())
				{
					bAllowed = false;
					OutReason = FText::Format(LOCTEXT("UnsupportedStructureTypeFormat", "User Defined Structs like {0} are not supported by this function"), FText::FromName(AttributeType->GetFName())).ToString();
				}
			}
		}
		else
		{
			OutReason =LOCTEXT("UnregisterdStructureType","Must be a registered Animation Attribute.").ToString();
		}

		return !bAllowed;
	}
	
	return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
}

UK2Node_SetAttributeKeyGeneric::UK2Node_SetAttributeKeyGeneric()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UAnimationAttributeBlueprintLibrary, SetAttributeKey), UAnimationAttributeBlueprintLibrary::StaticClass());
	AttributeActionFormat = LOCTEXT("SetAttributeKeyNodetitle", "Set {0} Key");
	AttributeValuePinName = ValuePinName;
}

UK2Node_SetAttributeKeysGeneric::UK2Node_SetAttributeKeysGeneric()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UAnimationAttributeBlueprintLibrary, SetAttributeKeys), UAnimationAttributeBlueprintLibrary::StaticClass());
	AttributeActionFormat = LOCTEXT("SetAttributeKesyNodetitle", "Set {0} Keys");
	AttributeValuePinName = ValuesPinName;
}

UK2Node_GetAttributeKeyGeneric::UK2Node_GetAttributeKeyGeneric()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UAnimationAttributeBlueprintLibrary, GetAttributeKey), UAnimationAttributeBlueprintLibrary::StaticClass());
	AttributeActionFormat = LOCTEXT("GetAttributeKeyNodetitle", "Get {0} Key");
	AttributeValuePinName = ValuePinName;
}

UK2Node_GetAttributeKeysGeneric::UK2Node_GetAttributeKeysGeneric()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UAnimationAttributeBlueprintLibrary, GetAttributeKeys), UAnimationAttributeBlueprintLibrary::StaticClass());
	AttributeActionFormat = LOCTEXT("GetAttributeKeysNodetitle", "Get {0} Keys");
	AttributeValuePinName = ValuesPinName;
}

#undef LOCTEXT_NAMESPACE
