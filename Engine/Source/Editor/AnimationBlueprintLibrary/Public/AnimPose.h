// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimSequenceHelpers.h"
#include "Animation/AnimationPoseData.h"
#include "AnimationBlueprintLibrary.h"
#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Math/Transform.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "AnimPose.generated.h"

#define UE_API ANIMATIONBLUEPRINTLIBRARY_API

class UAnimBlueprint;
class UAnimSequenceBase;
class UObject;
class USkeletalMesh;
class USkeletalMeshComponent;
class USkeleton;
struct FBoneContainer;
struct FCompactPose;
struct FFrame;

UENUM(BlueprintType)
enum class EAnimPoseSpaces : uint8
{
	// Local (bone) space 
	Local,
	// World (component) space
    World
};

UENUM(BlueprintType)
enum class EAnimDataEvalType : uint8
{
	// Evaluates the original Animation Source data 
	Source,
	// Evaluates the original Animation Source data with additive animation layers
	Raw,
	// Evaluates the compressed Animation data - matching runtime (cooked)
	Compressed
};

USTRUCT(BlueprintType)
struct FAnimPoseEvaluationOptions
{
	GENERATED_BODY()

	// Type of evaluation which should be used
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Pose")
	EAnimDataEvalType EvaluationType = EAnimDataEvalType::Raw;

	// Whether or not to retarget animation during evaluation
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category="Animation|Pose")
	bool bShouldRetarget = true;

	// Whether or not to extract root motion values
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category="Animation|Pose")
	bool bExtractRootMotion = false;

	// Whether or not to force root motion being incorporated into retrieved pose
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category="Animation|Pose")
	bool bIncorporateRootMotionIntoPose = true;

	// Optional skeletal mesh with proportions to use when evaluating a pose
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category="Animation|Pose")
	TObjectPtr<USkeletalMesh> OptionalSkeletalMesh = nullptr;
	
	// Whether or additive animations should be applied to their base-pose 
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category="Animation|Pose")
	bool bRetrieveAdditiveAsFullPose = true;

	// Whether or not to evaluate Animation Curves
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category="Animation|Pose")
	bool bEvaluateCurves = true;
};

/** Script friendly representation of an evaluated animation bone pose */
USTRUCT(BlueprintType)
struct FAnimPose
{
	GENERATED_BODY()

	/** Returns whether or not the pose data was correctly initialized and populated */
	UE_API bool IsValid() const;
	
protected:
	/** Initializes the various arrays, using and copying the provided bone container */
	UE_API void Init(const FBoneContainer& InBoneContainer);
	/** Populates an FCompactPose using the contained bone data */
	UE_API void GetPose(FCompactPose& InOutCompactPose) const;
	/** Generates the contained bone data using the provided Component and its AnimInstance */
	UE_API void SetPose(USkeletalMeshComponent* Component);
	/** Generates the contained bone data using the provided CompactPose */
	UE_API void SetPose(const FAnimationPoseData& PoseData);
	/** Copies the reference pose to animated pose data */
	UE_API void SetToRefPose();

	/** (Re-)Generates the world space transforms using populated local space data */
	UE_API void GenerateWorldSpaceTransforms();
	/** Resets all contained data, rendering the instance invalid */
	UE_API void Reset();

	/** Whether or not the contained data was initialized and can be used to store a pose */
	bool IsInitialized() const { return BoneNames.Num() != 0; }
	/** Whether or local space pose data has been populated */
	bool IsPopulated() const { return LocalSpacePoses.Num() != 0; }
	
protected:
	UPROPERTY()
	TArray<FName> BoneNames;
	
	UPROPERTY()
	TArray<int32> BoneIndices;

	UPROPERTY()
	TArray<int32> ParentBoneIndices;

	UPROPERTY()
	TArray<FTransform> LocalSpacePoses;
	
	UPROPERTY()
	TArray<FTransform> WorldSpacePoses;

	UPROPERTY()
	TArray<FTransform> RefLocalSpacePoses;
	
	UPROPERTY()
	TArray<FTransform> RefWorldSpacePoses;

	UPROPERTY()
	TArray<FName> CurveNames;

	UPROPERTY()
	TArray<float> CurveValues;
	
	UPROPERTY()
	TArray<FName> SocketNames;

	UPROPERTY()
	TArray<FName> SocketParentBoneNames;
	
	UPROPERTY()
	TArray<FTransform> SocketTransforms;

	friend class UAnimPoseExtensions;
};

/** Script exposed functionality for populating, retrieving data from and setting data on FAnimPose */
UCLASS(MinimalAPI)
class UAnimPoseExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
	
public:
	/**
	*  Returns whether the Anim Pose contains valid data
	*
	* @param	Pose	Anim Pose to validate
	*
	* @return	Result of the validation
	*/	
	UFUNCTION(BlueprintPure, meta = (ScriptMethod), Category = "Animation|Pose")
	static UE_API bool IsValid(UPARAM(ref) const FAnimPose& Pose);

	/**
	* Returns an array of bone names contained by the pose
	*
	* @param	Pose	Anim Pose to retrieve the names from
	* @param	Bones	Array to be populated with the bone names
	*
	*/
	UFUNCTION(BlueprintPure, meta = (ScriptMethod), Category = "Animation|Pose")
	static UE_API void GetBoneNames(UPARAM(ref) const FAnimPose& Pose, TArray<FName>& Bones);

	/**
	* Retrieves the transform for the provided bone name from a pose
	*
	* @param	Pose		Anim Pose to retrieve the transform from
	* @param	BoneName	Name of the bone to retrieve
	* @param	Space		Space in which the transform should be retrieved
	*
	* @return	Transform in requested space for bone if found, otherwise return identity transform
	*/
	UFUNCTION(BlueprintPure, meta = (ScriptMethod), Category = "Animation|Pose")
    static UE_API const FTransform& GetBonePose(UPARAM(ref) const FAnimPose& Pose, FName BoneName, EAnimPoseSpaces Space = EAnimPoseSpaces::Local);
	
	/**
	* Sets the transform for the provided bone name for a pose
	*
	* @param	Pose		Anim Pose to set transform in
	* @param	Transform	Transform to set the bone to
	* @param	BoneName	Name of the bone to set
	* @param	Space		Space in which the transform should be set
	*/
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "Animation|Pose")
	static UE_API void SetBonePose(UPARAM(ref) FAnimPose& Pose, FTransform Transform, FName BoneName, EAnimPoseSpaces Space = EAnimPoseSpaces::Local);	

	/**
	* Retrieves the reference pose transform for the provided bone name
	*
	* @param	Pose		Anim Pose to retrieve the transform from
	* @param	BoneName	Name of the bone to retrieve
	* @param	Space		Space in which the transform should be retrieved
	*
	* @return	Transform in requested space for bone if found, otherwise return identity transform
	*/
	UFUNCTION(BlueprintPure, meta = (ScriptMethod), Category = "Animation|Pose")
    static UE_API const FTransform& GetRefBonePose(UPARAM(ref) const FAnimPose& Pose, FName BoneName, EAnimPoseSpaces Space = EAnimPoseSpaces::Local);

	/**
	* Retrieves the relative transform between the two provided bone names
	*
	* @param	Pose			Anim Pose to retrieve the transform from
	* @param	FromBoneName	Name of the bone to retrieve the transform relative from
	* @param	ToBoneName		Name of the bone to retrieve the transform relative to
	* @param	Space			Space in which the transform should be retrieved
	*
	* @return	Relative transform in requested space for bone if found, otherwise return identity transform
	*/
	UFUNCTION(BlueprintPure, meta = (ScriptMethod), Category = "Animation|Pose")
    static UE_API FTransform GetRelativeTransform(UPARAM(ref) const FAnimPose& Pose, FName FromBoneName, FName ToBoneName, EAnimPoseSpaces Space = EAnimPoseSpaces::Local);

	/**
	* Retrieves the relative transform between reference and animated bone transform
	*
	* @param	Pose			Anim Pose to retrieve the transform from
	* @param	BoneName		Name of the bone to retrieve the relative transform for
	* @param	Space			Space in which the transform should be retrieved
	*
	* @return	Relative transform in requested space for bone if found, otherwise return identity transform
	*/
	UFUNCTION(BlueprintPure, meta = (ScriptMethod), Category = "Animation|Pose")
    static UE_API FTransform GetRelativeToRefPoseTransform(UPARAM(ref) const FAnimPose& Pose, FName BoneName, EAnimPoseSpaces Space = EAnimPoseSpaces::Local);

	/**
	* Retrieves the relative transform for the reference pose between the two provided bone names 
	*
	* @param	Pose			Anim Pose to retrieve the transform from
	* @param	FromBoneName	Name of the bone to retrieve the transform relative from
	* @param	ToBoneName		Name of the bone to retrieve the transform relative to
	* @param	Space			Space in which the transform should be retrieved
	*
	* @return	Relative transform in requested space for bone if found, otherwise return identity transform
	*/
	UFUNCTION(BlueprintPure, meta = (ScriptMethod), Category = "Animation|Pose")
    static UE_API FTransform GetRefPoseRelativeTransform(UPARAM(ref) const FAnimPose& Pose, FName FromBoneName, FName ToBoneName, EAnimPoseSpaces Space = EAnimPoseSpaces::Local);

	/**
	* Returns an array of socket names contained by the pose
	*
	* @param	Pose	Anim Pose to retrieve the names from
	* @param	Sockets	Array to be populated with the socket names
	*
	*/
	UFUNCTION(BlueprintPure, meta = (ScriptMethod), Category = "Animation|Pose")
	static UE_API void GetSocketNames(UPARAM(ref) const FAnimPose& Pose, TArray<FName>& Sockets);

	/**
	* Retrieves the transform for the provided socket name from a pose
	*
	* @param	Pose		Anim Pose to retrieve the transform from
	* @param	SocketName	Name of the socket to retrieve
	* @param	Space		Space in which the transform should be retrieved
	*
	* @return	Transform in requested space for bone if found, otherwise return identity transform
	*/
	UFUNCTION(BlueprintPure, meta = (ScriptMethod), Category = "Animation|Pose")
	static UE_API FTransform GetSocketPose(UPARAM(ref) const FAnimPose& Pose, FName SocketName, EAnimPoseSpaces Space = EAnimPoseSpaces::Local);
	

	/**
	* Evaluates an Animation Sequence Base to generate a valid Anim Pose instance
	*
	* @param	AnimationSequenceBase	Animation sequence base to evaluate the pose from
	* @param	Time					Time at which the pose should be evaluated
	* @param	EvaluationOptions		Options determining the way the pose should be evaluated
	* @param	Pose					Anim Pose to hold the evaluated data
	*/
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "Animation|Pose")
	static UE_API void GetAnimPoseAtTime(const UAnimSequenceBase* AnimationSequenceBase, double Time, FAnimPoseEvaluationOptions EvaluationOptions, FAnimPose& Pose);

	/**
	* Evaluates an Animation Sequence Base at different time intervals to generate a valid Anim Pose instances
	*
	* @param	AnimationSequenceBase	Animation sequence base to evaluate the pose from
	* @param	TimeIntervals			Times at which the pose should be evaluated
	* @param	EvaluationOptions		Options determining the way the pose should be evaluated
	* @param	InOutPoses				Anim Poses holding the evaluated data (number matches TimeIntervals)
	*/
	static UE_API void GetAnimPoseAtTimeIntervals(const UAnimSequenceBase* AnimationSequenceBase, TArray<double> TimeIntervals, FAnimPoseEvaluationOptions EvaluationOptions, TArray<FAnimPose>& InOutPoses);

	/**
	* Evaluates an Animation Sequence Base to generate a valid Anim Pose instance
	*
	* @param	AnimationSequenceBase	Animation sequence base to evaluate the pose from
	* @param	FrameIndex				Exact frame at which the pose should be evaluated
	* @param	EvaluationOptions		Options determining the way the pose should be evaluated
	* @param	Pose					Anim Pose to hold the evaluated data
	*/
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "Animation|Pose")
	static UE_API void GetAnimPoseAtFrame(const UAnimSequenceBase* AnimationSequenceBase, int32 FrameIndex, FAnimPoseEvaluationOptions EvaluationOptions, FAnimPose& Pose);

	/**
	* Evaluates an Animation Blueprint instance, using the provided Anim Pose and its Input Pose value, generating a valid Anim Pose using the result. Warning this function may cause performance issues.
	*
	* @param	InputPose				Anim Pose used to populate the input pose node value inside of the Animation Blueprint
	* @param	TargetSkeletalMesh		USkeletalMesh object used as the target skeletal mesh, should have same USkeleton as InputPose and Animation Blueprint
	* @param	AnimationBlueprint		Animation Blueprint to generate an AnimInstance with, used to evaluate the output Anim Pose
	* @param	OutPose					Anim pose to hold the data from evaluating the Animation Blueprint instance		
	*/
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "Animation|Pose")
	static UE_API void EvaluateAnimationBlueprintWithInputPose(const FAnimPose& InputPose, USkeletalMesh* TargetSkeletalMesh, UAnimBlueprint* AnimationBlueprint, FAnimPose& OutPose);

	/**
	* Populates an Anim Pose with the reference poses stored for the provided USkeleton
	*
	* @param	Skeleton				USkeleton object to retrieve the reference pose from
	* @param	OutPose					Anim pose to hold the reference pose
	*/
	UFUNCTION(BlueprintPure, meta = (ScriptMethod), Category = "Animation|Pose")
	static UE_API void GetReferencePose(USkeleton* Skeleton, FAnimPose& OutPose);

	/**
	* Returns an array of curve names contained by the pose
	*
	* @param	Pose	Anim Pose to retrieve the names from
	* @param	Curves	Array to be populated with the curve names
	*
	*/
	UFUNCTION(BlueprintPure, meta = (ScriptMethod), Category = "Animation|Pose")
	static UE_API void GetCurveNames(UPARAM(ref) const FAnimPose& Pose, TArray<FName>& Curves);
	
	/**
    * Returns the weight of an evaluated curve - if found
    *
    * @param	Pose		Anim Pose to retrieve the value from
    * @param	CurveName	Curve to retrieve the weight value for
    * 
    * @return	Curve weight value, if found - 0.f otherwise
    */
    UFUNCTION(BlueprintPure, meta = (ScriptMethod), Category = "Animation|Pose")
    static UE_API float GetCurveWeight(UPARAM(ref) const FAnimPose& Pose, const FName& CurveName);


	/**
    * Returns a CompactPose from the given FAnimPose.  This function should not be exposed to blueprint 
    *
    * @param	Pose		Anim Pose to retrieve the value from
    * @param	InOutCompactPose	CompactPose from the Pose
    */
	static UE_API void GetCompactPose(const FAnimPose& Pose, FCompactPose& InOutCompactPose);
};

#undef UE_API
