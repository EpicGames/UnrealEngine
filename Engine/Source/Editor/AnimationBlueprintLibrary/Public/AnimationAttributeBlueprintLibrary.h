// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimData/IAnimationDataController.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptInterface.h"
#include "UObject/UObjectGlobals.h"

#include "AnimationAttributeBlueprintLibrary.generated.h"

#define UE_API ANIMATIONBLUEPRINTLIBRARY_API

class FArrayProperty;
class IAnimationDataController;
class UAnimDataModel;
class UObject;
class UScriptStruct;
struct FAnimationAttributeIdentifier;
struct FFrame;

UCLASS(MinimalAPI, meta=(ScriptName="AnimationAttributeLibrary"))
class UAnimationAttributeBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "Animation|Attributes", meta=(CustomStructureParam="Value", BlueprintInternalUseOnly="true"))
	static UE_API bool SetAttributeKey(TScriptInterface<IAnimationDataController> AnimationDataController, const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, const int32& Value);
	static UE_API bool Generic_SetAttributeKey(TScriptInterface<IAnimationDataController> AnimationDataController, const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, UScriptStruct* ScriptStruct, const void* Value);
	DECLARE_FUNCTION(execSetAttributeKey);

	UFUNCTION(BlueprintCallable, CustomThunk, Category = "Animation|Attributes", meta=(CustomStructureParam="Values", BlueprintInternalUseOnly="true"))
	static UE_API bool SetAttributeKeys(TScriptInterface<IAnimationDataController> AnimationDataController, const FAnimationAttributeIdentifier& AttributeIdentifier, const TArray<float>& Times, const TArray<int32>& Values);
	static UE_API bool Generic_SetAttributeKeys(TScriptInterface<IAnimationDataController> AnimationDataController, const FAnimationAttributeIdentifier& AttributeIdentifier, const TArray<float>& Times, const void* ValuesArray, const FArrayProperty* ValuesArrayProperty);
	DECLARE_FUNCTION(execSetAttributeKeys);


	UFUNCTION(BlueprintCallable, CustomThunk, Category = "Animation|Attributes", meta=(CustomStructureParam="Value", BlueprintInternalUseOnly="true"))
	static UE_API bool GetAttributeKey(TScriptInterface<IAnimationDataModel> AnimationDataModel, const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, int32& Value);
	static UE_API bool Generic_GetAttributeKey(TScriptInterface<IAnimationDataModel> AnimationDataModel, const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, UScriptStruct* ScriptStruct, void* Value);
	DECLARE_FUNCTION(execGetAttributeKey);

	UFUNCTION(BlueprintCallable, CustomThunk, Category = "Animation|Attributes", meta=(CustomStructureParam="Values", BlueprintInternalUseOnly="true"))
	static UE_API bool GetAttributeKeys(TScriptInterface<IAnimationDataModel> AnimationDataModel, const FAnimationAttributeIdentifier& AttributeIdentifier, TArray<float>& OutTimes, TArray<int32>& Values);
	static UE_API bool Generic_GetAttributeKeys(TScriptInterface<IAnimationDataModel> AnimationDataModel, const FAnimationAttributeIdentifier& AttributeIdentifier, TArray<float>& Times, void* ValuesArray, const FArrayProperty* ValuesArrayProperty);
	DECLARE_FUNCTION(execGetAttributeKeys);
};

#undef UE_API
