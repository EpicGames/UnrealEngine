// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimEnums.h"
#include "Animation/AnimMetaData.h"
#include "Animation/AnimTypes.h"
#include "Animation/SmartName.h"
#include "AnimationGraph.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Delegates/Delegate.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Math/Color.h"
#include "Math/Quat.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"

#include "AnimationBlueprintLibrary.generated.h"

#define UE_API ANIMATIONBLUEPRINTLIBRARY_API

class UAnimBlueprint;
class UAnimBoneCompressionSettings;
class UAnimCompress;
class UAnimCurveCompressionSettings;
class UVariableFrameStrippingSettings;
class UAnimGraphNode_Base;
class UAnimMetaData;
class UAnimMontage;
class UAnimNotify;
class UAnimNotifyState;
class UAnimSequence;
class UAnimSequenceBase;
class UAnimationAsset;
class UAnimationGraph;
class UObject;
class USkeletalMesh;
class USkeleton;
struct FFrame;
struct FQualifiedFrameTime;
struct FRawAnimSequenceTrack;

UENUM()
enum class ESmartNameContainerType : uint8
{
	SNCT_CurveMapping UMETA(DisplayName = "Curve Names"),
	SNCT_TrackCurveMapping	UMETA(DisplayName = "Track Curve Names"),
	SNCT_MAX
};

/** Delegate called when a notify was replaced */
DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnNotifyReplaced, UAnimNotify*, OldNotify, UAnimNotify*, NewNotify);

/** Delegate called when a notify state was replaced */
DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnNotifyStateReplaced, UAnimNotifyState*, OldNotifyState, UAnimNotifyState*, NewNotifyState);

/** Blueprint library for altering and analyzing animation / skeletal data */
UCLASS(MinimalAPI, meta=(ScriptName="AnimationLibrary"))
class UAnimationBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Retrieves the number of animation frames for the given Animation Sequence */
	UFUNCTION(BlueprintPure, meta=(AutoCreateRefTerm = "AnimationSequence"), Category = "AnimationBlueprintLibrary|Animation")
	static UE_API void GetNumFrames(const UAnimSequenceBase* AnimationSequenceBase, int32& NumFrames);

	/** Retrieves the number of animation keys for the given Animation Sequence */
	UFUNCTION(BlueprintPure, meta = (AutoCreateRefTerm = "AnimationSequence"), Category = "AnimationBlueprintLibrary|Animation")
	static UE_API void GetNumKeys(const UAnimSequenceBase* AnimationSequenceBase, int32& NumKeys);
	
	/** Retrieves the Names of the individual ATracks for the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Animation")
	static UE_API void GetAnimationTrackNames(const UAnimSequenceBase* AnimationSequenceBase, TArray<FName>& TrackNames);

	/** Retrieves the Names of the Animation Slots used in the given Montage */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Montage")
	static UE_API void GetMontageSlotNames(const UAnimMontage* AnimationMontage, TArray<FName>& SlotNames);

	/** Retrieves the Names of the individual float curves for the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Animation")
	static UE_API void GetAnimationCurveNames(const UAnimSequenceBase* AnimationSequenceBase, ERawCurveTrackTypes CurveType, TArray<FName>& CurveNames);

	/** Gets the root transform from the raw animation at Time */
	UFUNCTION(BlueprintCallable, Category="AnimationBlueprintLibrary|Animation")
	static UE_API FTransform ExtractRootTrackTransform(const UAnimSequenceBase* AnimationSequenceBase, float Time);

	/** Retrieves the Raw Translation Animation Data for the given Animation Track Name and Animation Sequence */
	UE_DEPRECATED(5.2, "GetRawTrackPositionData has been deprecated, use AnimationModel interface instead")
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|RawTrackData")
	static void GetRawTrackPositionData(const UAnimSequenceBase* AnimationSequenceBase, const FName TrackName, TArray<FVector>& PositionData) {}

	/** Retrieves the Raw Rotation Animation Data for the given Animation Track Name and Animation Sequence */
	UE_DEPRECATED(5.2, "GetRawTrackRotationData has been deprecated, use AnimationModel interface instead")
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|RawTrackData")
	static void GetRawTrackRotationData(const UAnimSequenceBase* AnimationSequenceBase, const FName TrackName, TArray<FQuat>& RotationData ) {}

	/** Retrieves the Raw Scale Animation Data for the given Animation Track Name and Animation Sequence */	
	UE_DEPRECATED(5.2, "GetRawTrackScaleData has been deprecated, use AnimationModel interface instead")
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|RawTrackData")
	static void GetRawTrackScaleData(const UAnimSequenceBase* AnimationSequenceBase, const FName TrackName, TArray<FVector>& ScaleData) {}

	/** Retrieves the Raw Animation Data for the given Animation Track Name and Animation Sequence */
	UE_DEPRECATED(5.2, "GetRawTrackScaleData has been deprecated, use AnimationModel interface instead")
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|RawTrackData")
	static void GetRawTrackData(const UAnimSequenceBase* AnimationSequenceBase, const FName TrackName, TArray<FVector>& PositionKeys,TArray<FQuat>& RotationKeys, TArray<FVector>& ScalingKeys) {}

	/** Checks whether or not the given Animation Track Name is contained within the Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Helpers")
	static bool IsValidRawAnimationTrackName(const UAnimSequenceBase* AnimationSequenceBase, const FName TrackName) { return false; }

	// Compression

	/** Retrieves the Bone Compression Settings for the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Compression")
	static UE_API void GetBoneCompressionSettings(const UAnimSequence* AnimationSequence, UAnimBoneCompressionSettings*& CompressionSettings);

	/** Sets the Bone Compression Settings for the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Compression")
	static UE_API void SetBoneCompressionSettings(UAnimSequence* AnimationSequence, UAnimBoneCompressionSettings* CompressionSettings);

	/** Retrieves the Curve Compression Settings for the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Compression")
	static UE_API void GetCurveCompressionSettings(const UAnimSequence* AnimationSequence, UAnimCurveCompressionSettings*& CompressionSettings);

	/** Sets the Curve Compression Settings for the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Compression")
	static UE_API void SetCurveCompressionSettings(UAnimSequence* AnimationSequence, UAnimCurveCompressionSettings* CompressionSettings);

	/** Retrieves the Variable Frame Stripping Settings for the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Compression")
	static UE_API void GetVariableFrameStrippingSettings(const UAnimSequence* AnimationSequence, UVariableFrameStrippingSettings*& VariableFrameStrippingSettings);

	/** Sets the Variable Frame Stripping Settings for the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Compression")
	static UE_API void SetVariableFrameStrippingSettings(UAnimSequence* AnimationSequence, UVariableFrameStrippingSettings* VariableFrameStrippingSettings);

	// Additive 
	/** Retrieves the Additive Animation type for the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Additive")
	static UE_API void GetAdditiveAnimationType(const UAnimSequence* AnimationSequence, TEnumAsByte<enum EAdditiveAnimationType>& AdditiveAnimationType);

	/** Sets the Additive Animation type for the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Additive")
	static UE_API void SetAdditiveAnimationType(UAnimSequence* AnimationSequence, const TEnumAsByte<enum EAdditiveAnimationType> AdditiveAnimationType);

	/** Retrieves the Additive Base Pose type for the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Additive")
	static UE_API void GetAdditiveBasePoseType(const UAnimSequence* AnimationSequence, TEnumAsByte<enum EAdditiveBasePoseType>& AdditiveBasePoseType);

	/** Sets the Additive Base Pose type for the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Additive")
	static UE_API void SetAdditiveBasePoseType(UAnimSequence* AnimationSequence, const TEnumAsByte<enum EAdditiveBasePoseType> AdditiveBasePoseType);

	// Interpolation

	/** Retrieves the Animation Interpolation type for the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Interpolation")
	static UE_API void GetAnimationInterpolationType(const UAnimSequence* AnimationSequence, EAnimInterpolationType& InterpolationType);

	/** Sets the Animation Interpolation type for the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Interpolation")
	static UE_API void SetAnimationInterpolationType(UAnimSequence* AnimationSequence, EAnimInterpolationType InterpolationType);

	// Root motion

	/** Checks whether or not Root Motion is Enabled for the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|RootMotion")
	static UE_API bool IsRootMotionEnabled(const UAnimSequence* AnimationSequence);

	/** Sets whether or not Root Motion is Enabled for the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|RootMotion")
	static UE_API void SetRootMotionEnabled(UAnimSequence* AnimationSequence, bool bEnabled);

	/** Retrieves the Root Motion Lock Type for the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|RootMotion")
	static UE_API void GetRootMotionLockType(const UAnimSequence* AnimationSequence, TEnumAsByte<ERootMotionRootLock::Type>& LockType);

	/** Sets the Root Motion Lock Type for the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|RootMotion")
	static UE_API void SetRootMotionLockType(UAnimSequence* AnimationSequence, TEnumAsByte<ERootMotionRootLock::Type> RootMotionLockType);

	/** Checks whether or not Root Motion locking is Forced for the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|RootMotion")
	static UE_API bool IsRootMotionLockForced(const UAnimSequence* AnimationSequence);

	/** Sets whether or not Root Motion locking is Forced for the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|RootMotion")
	static UE_API void SetIsRootMotionLockForced(UAnimSequence* AnimationSequence, bool bForced);

	// Markers

	/** Retrieves all the Animation Sync Markers for the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|MarkerSyncing")
	static UE_API void GetAnimationSyncMarkers(const UAnimSequence* AnimationSequence, TArray<FAnimSyncMarker>& Markers);

	/** Retrieves all the Unique Names for the Animation Sync Markers contained by the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|MarkerSyncing")
	static UE_API void GetUniqueMarkerNames(const UAnimSequence* AnimationSequence, TArray<FName>& MarkerNames);

	/** Adds an Animation Sync Marker to Notify track in the given Animation with the corresponding Marker Name and Time */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|MarkerSyncing")
	static UE_API void AddAnimationSyncMarker(UAnimSequence* AnimationSequence, FName MarkerName, float Time, FName NotifyTrackName);

	/** Checks whether or not the given Marker Name is a valid Animation Sync Marker Name */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Helpers")
	static UE_API bool IsValidAnimationSyncMarkerName(const UAnimSequence* AnimationSequence, FName MarkerName);

	/** Removes All Animation Sync Marker found within the Animation Sequence whose name matches MarkerName, and returns the number of removed instances */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|MarkerSyncing")
	static UE_API int32 RemoveAnimationSyncMarkersByName(UAnimSequence* AnimationSequence, FName MarkerName);

	/** Removes All Animation Sync Marker found within the Animation Sequence that belong to the specific Notify Track, and returns the number of removed instances */	
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|MarkerSyncing")
	static UE_API int32 RemoveAnimationSyncMarkersByTrack(UAnimSequence* AnimationSequence, FName NotifyTrackName);

	/** Removes All Animation Sync Markers found within the Animation Sequence, and returns the number of removed instances */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|MarkerSyncing")
	static UE_API void RemoveAllAnimationSyncMarkers(UAnimSequence* AnimationSequence);

	// Notifies

	/** Retrieves all Animation Notify Events found within the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|NotifyEvents")
	static UE_API void GetAnimationNotifyEvents(const UAnimSequenceBase* AnimationSequenceBase, TArray<FAnimNotifyEvent>& NotifyEvents);

	/** Retrieves all Unique Animation Notify Events found within the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|NotifyEvents")
	static UE_API void GetAnimationNotifyEventNames(const UAnimSequenceBase* AnimationSequenceBase, TArray<FName>& EventNames);

	/** Adds an Animation Notify Event to Notify track in the given Animation with the given Notify creation data */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|NotifyEvents")
	static UE_API UAnimNotify* AddAnimationNotifyEvent(UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName, float StartTime, TSubclassOf<UAnimNotify> NotifyClass);
	
	UE_EXPERIMENTAL(5.7, "Adding a notify to an animation sequence from a FAnimNotifyEvent is expertimental.")
	/**
	 * Experimental: Adds a custom animation notify event to the specified animation sequence at the given time and on the specified notify track.
	 * 
	 * A new notify event will be created using relevant data copied from the provided NotifyEvent. The UAnimNotify is extracted from
	 * NotifyEvent and, if it is not already owned by the animation sequence (not part of its outer), it will be automatically duplicated.
	 * Only properties exposed to Blueprints in the NotifyEvent (such as TriggerWeightThreshold and NotifyTriggerChance) will be copied.
	 * 
	 * @param AnimationSequenceBase  The animation sequence to which the new notify event will be added.
	 * @param StartTime              The time (in seconds) within the sequence where the notify should be placed.
	 * @param NotifyEvent            The source notify event used to initialize the new one. Its UAnimNotify will be extracted and validated.
	 * @param NotifyTrackName        The name of the notify track to add the notify to.
	 * @return                       The UAnimNotify used in the new notify event (original or duplicated).
	 */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|NotifyEvents")
	static UAnimNotify* AddAnimationNotifyEventFromSource(UAnimSequenceBase* AnimationSequenceBase, float StartTime, FAnimNotifyEvent NotifyEvent, FName NotifyTrackName);
	
	/** Adds an Animation Notify State Event to Notify track in the given Animation with the given Notify State creation data */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|NotifyEvents")
	static UE_API UAnimNotifyState* AddAnimationNotifyStateEvent(UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName, float StartTime, float Duration, TSubclassOf<UAnimNotifyState> NotifyStateClass);

	/** Adds the specific Animation Notify to the Animation Sequence (requires Notify's outer to be the Animation Sequence) */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|NotifyEvents")
	static UE_API void AddAnimationNotifyEventObject(UAnimSequenceBase* AnimationSequenceBase, float StartTime, UAnimNotify* Notify, FName NotifyTrackName);
	
	/** Adds the specific Animation Notify State to the Animation Sequence (requires Notify State's outer to be the Animation Sequence) */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|NotifyEvents")
	static UE_API void AddAnimationNotifyStateEventObject(UAnimSequenceBase* AnimationSequenceBase, float StartTime, float Duration, UAnimNotifyState* NotifyState, FName NotifyTrackName);

	/** Removes Animation Notify Events found by Name within the Animation Sequence, and returns the number of removed name instances */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|NotifyEvents")
	static UE_API int32 RemoveAnimationNotifyEventsByName(UAnimSequenceBase* AnimationSequenceBase, FName NotifyName);

	/** Removes Animation Notify Events found by Track within the Animation Sequence, and returns the number of removed name instances */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|NotifyEvents")
	static UE_API int32 RemoveAnimationNotifyEventsByTrack(UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName);	

	/** Replaces animation notifies in the specified Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|NotifyEvents")
	static UE_API void ReplaceAnimNotifyStates(UAnimSequenceBase* AnimationSequenceBase, TSubclassOf<UAnimNotifyState> OldNotifyClass, TSubclassOf<UAnimNotifyState> NewNotifyClass, FOnNotifyStateReplaced OnNotifyStateReplaced);

	/** Replaces animation notifies in the specified Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|NotifyEvents")
	static UE_API void ReplaceAnimNotifies(UAnimSequenceBase* AnimationSequenceBase, TSubclassOf<UAnimNotify> OldNotifyClass, TSubclassOf<UAnimNotify> NewNotifyClass, FOnNotifyReplaced OnNotifyReplaced);

	/** Copies animation notifies from Src Animation Sequence to Dest. Creates anim notify tracks as necessary. Returns true on success. */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|NotifyEvents")
	static UE_API void CopyAnimNotifiesFromSequence(UAnimSequenceBase* SourceAnimationSequenceBase, UAnimSequenceBase* DestinationAnimationSequenceBase, bool bDeleteExistingNotifies = false);

	// Notify Tracks

	/** Retrieves all Unique Animation Notify Track Names found within the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|AnimationNotifies")
	static UE_API void GetAnimationNotifyTrackNames(const UAnimSequenceBase* AnimationSequenceBase, TArray<FName>& TrackNames);

	/** Adds an Animation Notify Track to the Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|AnimationNotifies")
	static UE_API void AddAnimationNotifyTrack(UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName, FLinearColor TrackColor = FLinearColor::White);

	/** Removes an Animation Notify Track from Animation Sequence by Name */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|AnimationNotifies")
	static UE_API void RemoveAnimationNotifyTrack(UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName);

	/** Removes All Animation Notify Tracks from Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|AnimationNotifies")
	static UE_API void RemoveAllAnimationNotifyTracks(UAnimSequenceBase* AnimationSequenceBase);

	/** Checks whether or not the given Track Name is a valid Animation Notify Track in the Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Helpers")
	static UE_API bool IsValidAnimNotifyTrackName(const UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName);

	static UE_API int32 GetTrackIndexForAnimationNotifyTrackName(const UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName);
	static UE_API const FAnimNotifyTrack& GetNotifyTrackByName(const UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName);

	/** Returns the actual trigger time for a NotifyEvent */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|AnimationNotifies")
	static UE_API float GetAnimNotifyEventTriggerTime(const FAnimNotifyEvent& NotifyEvent);
	
	/** Returns the duration for a NotifyEvent, only non-zero for Anim Notify States */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|AnimationNotifies")
	static UE_API float GetAnimNotifyEventDuration(const FAnimNotifyEvent& NotifyEvent);

	/** Retrieves all Animation Sync Markers for the given Notify Track Name from the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|MarkerSyncing")
	static UE_API void GetAnimationSyncMarkersForTrack(const UAnimSequence* AnimationSequence, FName NotifyTrackName, TArray<FAnimSyncMarker>& Markers);

	/** Retrieves all Animation Notify Events for the given Notify Track Name from the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|NotifyEvents")
	static UE_API void GetAnimationNotifyEventsForTrack(const UAnimSequenceBase* AnimationSequenceBase, FName NotifyTrackName, TArray<FAnimNotifyEvent>& Events);

	// Curves

	/** Adds an Animation Curve by Type and Name to the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Curves")
	static UE_API void AddCurve(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, ERawCurveTrackTypes CurveType = ERawCurveTrackTypes::RCT_Float, bool bMetaDataCurve = false);

	/** Removes an Animation Curve by Name from the given Animation Sequence (Raw Animation Curves [Names] may not be removed from the Skeleton) */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Curves")
	static UE_API void RemoveCurve(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, bool bRemoveNameFromSkeleton = false);

	/** Removes all Animation Curve Data from the given Animation Sequence (Raw Animation Curves [Names] may not be removed from the Skeleton) */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Curves")
	static UE_API void RemoveAllCurveData(UAnimSequenceBase* AnimationSequenceBase);

	/** Adds a Transformation Key to the specified Animation Curve inside of the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Curves")
	static UE_API void AddTransformationCurveKey(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, const float Time, const FTransform& Transform);

	/** Adds a multiple of Transformation Keys to the specified Animation Curve inside of the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Curves")
	static UE_API void AddTransformationCurveKeys(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, const TArray<float>& Times, const TArray<FTransform>& Transforms);

	/** Adds a Float Key to the specified Animation Curve inside of the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Curves")
	static UE_API void AddFloatCurveKey(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, const float Time, const float Value);

	/** Adds a multiple of Float Keys to the specified Animation Curve inside of the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Curves")
	static UE_API void AddFloatCurveKeys(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, const TArray<float>& Times, const TArray<float>& Values);

	/** Adds a Vector Key to the specified Animation Curve inside of the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Curves")
	static UE_API void AddVectorCurveKey(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, const float Time, const FVector Vector);

	/** Adds a multiple of Vector Keys to the specified Animation Curve inside of the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Curves")
	static UE_API void AddVectorCurveKeys(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, const TArray<float>& Times, const TArray<FVector>& Vectors);

	// Curve helper functions
	template <typename DataType, typename CurveClass, ERawCurveTrackTypes CurveType>
	static void AddCurveKeysInternal(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, const TArray<float>& Times, const TArray<DataType>& KeyData);

	// Returns true if successfully added, false if it was already existing
	static UE_API bool AddCurveInternal(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, int32 CurveFlags, ERawCurveTrackTypes SupportedCurveType);
	static UE_API bool RemoveCurveInternal(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, ERawCurveTrackTypes SupportedCurveType);

	/** Checks whether or not the given Bone Name exist on the Skeleton referenced by the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Skeleton")
	static UE_API void DoesBoneNameExist(UAnimSequence* AnimationSequence, FName BoneName, bool& bExists);

	static UE_API bool DoesBoneNameExistInternal(USkeleton* Skeleton, FName BoneName);

	/** @return the number of curve metadata entries on the skeleton **/
	UFUNCTION(BlueprintPure, meta = (ScriptMethod), Category = "AnimationBlueprintLibrary|Skeleton")
	static UE_API int32 GetNumCurveMetaData(const USkeleton* Skeleton);

	/**
	 * Adds a curve metadata entry with the specified name on the skeleton
	 * @param	InCurveName			The name of the curve to find
	 * @param   bTransact           If true record a new transaction
	 * @return true if an entry was added, false if an entry already existed
	 */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "AnimationBlueprintLibrary|Skeleton")
	static UE_API bool AddCurveMetaData(USkeleton* Skeleton, FName CurveName, bool bTransact = true);

	/**
	 * Get an array of all curve metadata names
	 * @param	OutNames	The array to receive the metadata names 
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod), Category = "AnimationBlueprintLibrary|Skeleton")
	static UE_API void GetCurveMetaDataNames(const USkeleton* Skeleton, TArray<FName>& OutNames);

	/**
	 * Renames a curve metadata entry. Metadata is preserved, but assigned to a different curve name.
	 * @param OldName	The name of an existing curve entry
	 * @param NewName	The name to change the entry to
	 * @return			true if the rename was successful (the old name was found and the new name didnt collide with an
	 *					existing entry)
	*/
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "AnimationBlueprintLibrary|Skeleton")
	static UE_API bool RenameCurveMetaData(USkeleton* Skeleton, FName OldName, FName NewName);

	/**
	 * Removes a curve metadata entry for the specified name.
	 * @param CurveName	The name of the curve to remove the metadata for
	 * @return true if the entry was successfully removed (i.e. it existed)
	 */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "AnimationBlueprintLibrary|Skeleton")
	static UE_API bool RemoveCurveMetaData(USkeleton* Skeleton, FName CurveName);

	/**
	 * Gets the material flag for a curve's metadata
	 * @param	CurveName	The name of the curve to check
	 * @return true if the flag has been set
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod), Category = "AnimationBlueprintLibrary|Skeleton")
	static UE_API bool GetCurveMetaDataMaterial(const USkeleton* Skeleton, FName CurveName);

	/**
	 * Gets the morph target flag for a curve's metadata
	 * @param	CurveName	The name of the curve to check
	 * @return true if the flag has been set
	 */
	UFUNCTION(BlueprintPure, meta = (ScriptMethod), Category = "AnimationBlueprintLibrary|Skeleton")
	static UE_API bool GetCurveMetaDataMorphTarget(const USkeleton* Skeleton, FName CurveName);

	/**
	 * Set the material flag for a curve's metadata
	 * @param	CurveName			The name of the curve to set
	 * @param	bOverrideMaterial	Whether to set the material flag
	 */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "AnimationBlueprintLibrary|Skeleton")
	static UE_API void SetCurveMetaDataMaterial(USkeleton* Skeleton, FName CurveName, bool bOverrideMaterial);

	/**
	 * Set the morph target flag for a curve's metadata
	 * @param	CurveName				The name of the curve to set
	 * @param	bOverrideMorphTarget	Whether to set the morph target flag
	 */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "AnimationBlueprintLibrary|Skeleton")
	static UE_API void SetCurveMetaDataMorphTarget(USkeleton* Skeleton, FName CurveName, bool bOverrideMorphTarget);

	/** Retrieves, a multiple of, Float Key(s) from the specified Animation Curve inside of the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Curves")
	static UE_API void GetFloatKeys(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, TArray<float>& Times, TArray<float>& Values);

	/** Retrieves, a multiple of, Vector Key(s) from the specified Animation Curve inside of the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Curves")
	static UE_API void GetVectorKeys(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, TArray<float>& Times, TArray<FVector>& Values);

	/** Retrieves, a multiple of, Transformation Key(s) from the specified Animation Curve inside of the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Curves")
	static UE_API void GetTransformationKeys(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, TArray<float>& Times, TArray<FTransform>& Values);

	/** Retrieves an evaluated float value for a given time from the specified Animation Curve inside of the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Curves")
	static UE_API float GetFloatValueAtTime(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, float Time);
	
	template <typename DataType, typename CurveClass, ERawCurveTrackTypes CurveType>
	static void GetCurveKeysInternal(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, TArray<float>& Times, TArray<DataType>& KeyData);

	/** Ensures that any curve names that do not exist on the NewSkeleton are added to it, in which case the SmartName on the actual curve itself will also be updated */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Curves", meta=(DeprecatedFunction, DeprecationMessage="It is no longer necessary to copy curve names to the skeleton. If metadata is required to be updated, please use the metadata setting APIs."))
	static void CopyAnimationCurveNamesToSkeleton(USkeleton* OldSkeleton, USkeleton* NewSkeleton, UAnimSequenceBase* SequenceBase, ERawCurveTrackTypes CurveType) {}
	
	// Bone Tracks

	/** Removes an Animation Curve by Name from the given Animation Sequence (Raw Animation Curves [Names] may not be removed from the Skeleton) 
	 * 
	 *	@param AnimationSequence : AnimSequence
	 *	@param BoneName : Name of bone track user wants to remove
	 *	@param bIncludeChildren : true if user wants to include all children of BoneName
	 *  @param bFinalize : If you set this to true, it will trigger compression. If you set bFinalize to be false, you'll have to manually trigger Finalize. 
	 */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Bones")
	static UE_API void RemoveBoneAnimation(UAnimSequence* AnimationSequence, FName BoneName, bool bIncludeChildren = true, bool bFinalize = true);

	/** Removes an Animation Curve by Name from the given Animation Sequence (Raw Animation Curves [Names] may not be removed from the Skeleton) 
	 * 
	 *	@param AnimationSequence : AnimSequence
	 *	@param BoneName : Name of bone track user wants to remove
	 *	@param bIncludeChildren : true if user wants to include all children of BoneName
	 *  @param ChildrenExcluded : If bIncludeChildren is true, this is an optional list of child bones you want to exclude.
	 *  @param bExcludeRecursively : If bChildrenExcluded is not empty, this will also exclude the descendants of bones in that list.
	 *  @param bFinalize : If you set this to true, it will trigger compression. If you set bFinalize to be false, you'll have to manually trigger Finalize.
	 */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Bones", meta=(AutoCreateRefTerm="ChildrenExcluded"))
	static UE_API void RemoveBoneSelectiveAnimation(UAnimSequence* AnimationSequence, FName BoneName, TArray<FName> ChildrenExcluded, bool bIncludeChildren = true, bool bExcludeRecursively = false, bool bFinalize = true);

	/** Removes all Animation Bone Track Data from the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Curves")
	static UE_API void RemoveAllBoneAnimation(UAnimSequence* AnimationSequence);

	/** Apply all the changes made to Bone Tracks to Finalize. This triggers recompression. Note that this is expensive, but will require to get correct compressed data */
	UE_DEPRECATED(5.0, "FinalizeBoneAnimation has been deprecated, use UAnimDataController instead")
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Curves", meta=(DeprecatedFunction, DeprecationMessage="FinalizeBoneAnimation has been deprecated, use UAnimDataController instead"))
	static void FinalizeBoneAnimation(UAnimSequence* AnimationSequence) {}

	// Smart name helper functions

	/** Checks whether or not the given Curve Name exist on the Skeleton referenced by the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Curves")
	static UE_API bool DoesCurveExist(UAnimSequenceBase* AnimationSequenceBase, FName CurveName, ERawCurveTrackTypes CurveType);

	static UE_API ERawCurveTrackTypes RetrieveCurveTypeForCurve(const UAnimSequenceBase* AnimationSequenceBase, FName CurveName);

	// MetaData

	/** Creates and Adds an instance of the specified MetaData Class to the given Animation Asset */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|MetaData")
	static UE_API void AddMetaData(UAnimationAsset* AnimationAsset, TSubclassOf<UAnimMetaData> MetaDataClass, UAnimMetaData*& MetaDataInstance);

	/** Adds an instance of the specified MetaData Class to the given Animation Asset (requires MetaDataObject's outer to be the Animation Asset) */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|MetaData")
	static UE_API void AddMetaDataObject(UAnimationAsset* AnimationAsset, UAnimMetaData* MetaDataObject);

	/** Removes all Meta Data from the given Animation Asset */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|MetaData")
	static UE_API void RemoveAllMetaData(UAnimationAsset* AnimationAsset);

	/** Removes the specified Meta Data Instance from the given Animation Asset */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|MetaData")
	static UE_API void RemoveMetaData(UAnimationAsset* AnimationAsset, UAnimMetaData* MetaDataObject);

	/** Removes all Meta Data Instance of the specified Class from the given Animation Asset */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|MetaData")
	static UE_API void RemoveMetaDataOfClass(UAnimationAsset* AnimationAsset, TSubclassOf<UAnimMetaData> MetaDataClass);

	/** Retrieves all Meta Data Instances from the given Animation Asset */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|MetaData")
	static UE_API void GetMetaData(const UAnimationAsset* AnimationAsset, TArray<UAnimMetaData*>& MetaData);

	/** Retrieves all Meta Data Instances from the given Animation Asset */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|MetaData")
	static UE_API void GetMetaDataOfClass(const UAnimationAsset* AnimationAsset, TSubclassOf<UAnimMetaData> MetaDataClass, TArray<UAnimMetaData*>& MetaDataOfClass);

	/** Checks whether or not the given Animation Asset contains Meta Data Instance of the specified Meta Data Class */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|MetaData")
	static UE_API bool ContainsMetaDataOfClass(const UAnimationAsset* AnimationAsset, TSubclassOf<UAnimMetaData> MetaDataClass);

	// Poses

	/** Retrieves Bone Pose data for the given Bone Name at the specified Time from the given Animation Sequence */
	UE_DEPRECATED(5.2, "GetBonePosesForTime is deprecated, use AnimPose or AnimationDataModel interface directly")
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Pose")
	static UE_API void GetBonePoseForTime(const UAnimSequenceBase* AnimationSequenceBase, FName BoneName, float Time, bool bExtractRootMotion, FTransform& Pose);

	/** Retrieves Bone Pose data for the given Bone Name at the specified Frame from the given Animation Sequence */
	UE_DEPRECATED(5.2, "GetBonePosesForTime is deprecated, use AnimPose or AnimationDataModel interface directly")
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Pose")
	static UE_API void GetBonePoseForFrame(const UAnimSequenceBase* AnimationSequenceBase, FName BoneName, int32 Frame, bool bExtractRootMotion, FTransform& Pose);

	/** Retrieves Bone Pose data for the given Bone Names at the specified Time from the given Animation Sequence */
	UE_DEPRECATED(5.2, "GetBonePosesForTime is deprecated, use AnimPose or AnimationDataModel interface directly")
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Pose")
	static UE_API void GetBonePosesForTime(const UAnimSequenceBase* AnimationSequenceBase, TArray<FName> BoneNames, float Time, bool bExtractRootMotion, TArray<FTransform>& Poses, const USkeletalMesh* PreviewMesh = nullptr);

	/** Retrieves Bone Pose data for the given Bone Names at the specified Frame from the given Animation Sequence */
	UE_DEPRECATED(5.2, "GetBonePosesForTime is deprecated, use AnimPose or AnimationDataModel interface directly")
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Pose")
	static UE_API void GetBonePosesForFrame(const UAnimSequenceBase* AnimationSequenceBase, TArray<FName> BoneNames, int32 Frame, bool bExtractRootMotion, TArray<FTransform>& Poses, const USkeletalMesh* PreviewMesh = nullptr);

	// Virtual bones

	/** Adds a Virtual Bone between the Source and Target Bones to the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|VirtualBones")
	static UE_API void AddVirtualBone(const UAnimSequence* AnimationSequence, FName SourceBoneName, FName TargetBoneName, FName& VirtualBoneName);

	/** Removes a Virtual Bone with the specified Bone Name from the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|VirtualBones")
	static UE_API void RemoveVirtualBone(const UAnimSequence* AnimationSequence, FName VirtualBoneName);

	/** Removes Virtual Bones with the specified Bone Names from the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|VirtualBones")
	static UE_API void RemoveVirtualBones(const UAnimSequence* AnimationSequence, TArray<FName> VirtualBoneNames);

	/** Removes all Virtual Bones from the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|VirtualBones")
	static UE_API void RemoveAllVirtualBones(const UAnimSequence* AnimationSequence);

	static UE_API bool DoesVirtualBoneNameExistInternal(USkeleton* Skeleton, FName BoneName);

	// Misc

	/** Retrieves the Length of the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Animation")
	static UE_API void GetSequenceLength(const UAnimSequenceBase* AnimationSequenceBase, float& Length);

	/** Retrieves the (Play) Rate Scale of the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Animation")
	static UE_API void GetRateScale(const UAnimSequenceBase* AnimationSequenceBase, float& RateScale);

	/** Sets the (Play) Rate Scale for the given Animation Sequence */
	UFUNCTION(BlueprintCallable, Category = "AnimationBlueprintLibrary|Animation")
	static UE_API void SetRateScale(UAnimSequenceBase* AnimationSequenceBase, float RateScale);

	/** Retrieves the Frame Index at the specified Time Value for the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Helpers")
	static UE_API void GetFrameAtTime(const UAnimSequenceBase* AnimationSequenceBase, const float Time, int32& Frame);

	/** Retrieves the Time Value at the specified Frame Indexfor the given Animation Sequence */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Helpers")
	static UE_API void GetTimeAtFrame(const UAnimSequenceBase* AnimationSequenceBase, const int32 Frame, float& Time);
	
	static UE_API float GetTimeAtFrameInternal(const UAnimSequenceBase* AnimationSequenceBase, const int32 Frame);

	/** Checks whether or not the given Time Value lies within the given Animation Sequence's Length */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Helpers")
	static UE_API void IsValidTime(const UAnimSequenceBase* AnimationSequenceBase, const float Time, bool& IsValid);

	static UE_API bool IsValidTimeInternal(const UAnimSequenceBase* AnimationSequenceBase, const float Time);

	/**
	 * Given an animation sequence determine if any bones in that animation sequence has timecode attribute data.
	 *
	 * @return The bone name if found; otherwise none value will be returned.
	 */
	static UE_API FName FindBoneNameWithTimecodeAttributes(const UAnimSequenceBase* AnimSequenceBase);

	/** Evaluates timecode attributes (e.g. "TCFrame", "TCSecond", etc.) of the root bone and returns the resulting qualified frame time.
	 *
	 *  @param AnimationSequenceBase: Anim sequence for which to evaluate the root bone attributes.
	 *  @param EvalTime: Time (in seconds) at which to evaluate the timecode bone attributes.
	 *  @param OutQualifiedFrameTime: Resulting qualified frame time from evaluation. If the anim sequence has an import file frame rate
	 *      set, then that will be used as the frame rate of the qualified frame time. Otherwise, the sampling frame rate of the anim
	 *      sequence is used. If no timecode attributes are present on the bone or if none can be evaluated, the passed object will not be modified.
	 *  @return: true if the root bone had timecode attributes that could be evaluated and a qualified frame time was set, or false otherwise.
	 */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Helpers")
	static UE_API bool EvaluateRootBoneTimecodeAttributesAtTime(const UAnimSequenceBase* AnimationSequenceBase, const float EvalTime, FQualifiedFrameTime& OutQualifiedFrameTime);

	/** Evaluates timecode attributes (e.g. "TCFrame", "TCSecond", etc.) and TCSlate of the root bone and returns the resulting qualified frame time.
	 *
	 *  @param AnimationSequenceBase: Anim sequence for which to evaluate the root bone attributes.
	 *  @param EvalTime: Time (in seconds) at which to evaluate the timecode bone attributes.
	 *  @param OutQualifiedFrameTime: Resulting qualified frame time from evaluation. If the anim sequence has an import file frame rate
	 *      set, then that will be used as the frame rate of the qualified frame time. Otherwise, the sampling frame rate of the anim
	 *      sequence is used. If no timecode attributes are present on the bone or if none can be evaluated, the passed object will not be modified.
	 *  @param OutSlate:  The slate name (if any).  If so TCSlate exists on the root bone then an empty string is returned.
	 *
	 *  @return: true if the root bone had timecode attributes that could be evaluated and a qualified frame time was set, or false otherwise.
	 */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Helpers")
	static UE_API bool EvaluateBoneTimecodeAndSlateAttributesAtTime(const FName BoneName, const UAnimSequenceBase* AnimationSequenceBase, const float EvalTime, FQualifiedFrameTime& OutQualifiedFrameTime, FString& Slate);

	/** Evaluates the subframe timecode attribute (e.g. "TCSubframe") of the root bone and returns the resulting value.
	 *
	 *  Since the subframe component of FFrameTime is clamped to the range [0.0, 1.0), it cannot accurately represent the use
	 *  case where the timecode metadata represents subframe values as whole numbered subframes instead of as a percentage of a
	 *  frame the way the engine does. The subframe component of the FQualifiedFrameTime returned by
	 *  EvaluateRootBoneTimecodeAttributesAtTime() may not reflect the authored subframe metadata in that case.
	 * 
	 *  This function allows access to the subframe values that were actually authored in the timecode metadata.
	 *
	 *  @param AnimationSequenceBase: Anim sequence for which to evaluate the root bone subframe attribute.
	 *  @param EvalTime: Time (in seconds) at which to evaluate the subframe timecode bone attribute.
	 *  @param OutSubframe: Resulting subframe value from evaluation. If no subframe timecode attribute is present
	 *      on the bone or if it cannot be evaluated, the output parameter will not be modified.
	 *  @return: true if the root bone had a subframe timecode attribute that could be evaluated and a value was set, or false otherwise.
	 */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Helpers")
	static UE_API bool EvaluateRootBoneTimecodeSubframeAttributeAtTime(const UAnimSequenceBase* AnimationSequenceBase, const float EvalTime, float& OutSubframe);

	/** Finds the Bone Path from the given Bone to the Root Bone */
	UFUNCTION(BlueprintPure, Category = "AnimationBlueprintLibrary|Helpers")
	static UE_API void FindBonePathToRoot(const UAnimSequenceBase* AnimationSequenceBase, FName BoneName, TArray<FName>& BonePath);

	/** Returns all Animation Graphs contained by the provided Animation Blueprint */
	UFUNCTION(BlueprintCallable, Category=Animation, meta=(ScriptMethod))
	static UE_API void GetAnimationGraphs(UAnimBlueprint* AnimationBlueprint, TArray<UAnimationGraph*>& AnimationGraphs);

	/** Returns all Animation Graph Nodes of the provided Node Class contained by the Animation Blueprint */
	UFUNCTION(BlueprintCallable, Category=Animation, meta=(ScriptMethod))
	static UE_API void GetNodesOfClass(UAnimBlueprint* AnimationBlueprint, TSubclassOf<UAnimGraphNode_Base> NodeClass, TArray<UAnimGraphNode_Base*>& GraphNodes, bool bIncludeChildClasses = true);

	/**
	 * Adds an Animation Asset override for the provided AnimationBlueprint, replacing any instance of Target with Override
	 *
	 * @param AnimBlueprint					The Animation Blueprint to add/set the Override for
	 * @param Target						The Animation Asset to add an override for (overrides all instances of the asset)
	 * @param Override						The Animation Asset to used to override the Target with (types have to match)
	 * @param bPrintAppliedOverrides		Flag whether or not to print the applied overrides
	 */
	UFUNCTION(BlueprintCallable, Category=Animation, meta = (ScriptMethod))
	static UE_API void AddNodeAssetOverride(UAnimBlueprint* AnimBlueprint, const UAnimationAsset* Target, UAnimationAsset* Override, bool bPrintAppliedOverrides = false);

	/** Sets the preview skeletal mesh on a skeleton, optionally marks the skeleton as dirty */
	UFUNCTION(BlueprintCallable, Category=Skeleton, meta = (ScriptMethod))
	static UE_API void SetSkeletonPreviewMesh(USkeleton* Skeleton, USkeletalMesh* PreviewMesh, bool bMarkAsDirty=true);

	/** Returns the preview skeletal mesh on a skeleton, optionally finds one if a preview mesh hasn't been set */
	UFUNCTION(BlueprintCallable, Category=Skeleton, meta = (ScriptMethod))
	static UE_API USkeletalMesh* GetSkeletonPreviewMesh(USkeleton* Skeleton, bool bFindIfNotSet = false);

	/** Sets the additional preview skeletal meshes on a skeleton */
	UFUNCTION(BlueprintCallable, Category=Skeleton, meta = (ScriptMethod))
	static UE_API void SetSkeletonAdditionalPreviewMeshes(USkeleton* Skeleton, UDataAsset* AdditionalMeshes);

	/** Returns the additional preview skeletal meshes on a skeleton */
	UFUNCTION(BlueprintPure, Category=Skeleton, meta = (ScriptMethod))
	static UE_API UDataAsset* GetSkeletonAdditionalPreviewMeshes(const USkeleton* Skeleton);
};

#undef UE_API
