// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "K2Node_CallFunction.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

#include "K2_AnimationAttributeNodes.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FString;
class UEdGraph;
class UEdGraphPin;
class UObject;

/** Base node implementation to reduce duplicated behaviour for different BlueprintLibrary functions */
UCLASS(abstract)
class UK2Node_BaseAttributeActionNode : public UK2Node_CallFunction
{
	GENERATED_BODY()

public:
	UK2Node_BaseAttributeActionNode() {}
	
	//~ Begin UEdGraphNode Interface.
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	//~ End UK2Node Interface

protected:
	//~ Begin UK2Node_CallFunction interface
	virtual bool CanToggleNodePurity() const override { return false; }
	//~ End UK2Node_CallFunction interface

	FName AttributeValuePinName = NAME_None;
	FText AttributeActionFormat;
	
	static FName ValuePinName;
	static FName ValuesPinName;
};

UCLASS()
class UK2Node_SetAttributeKeyGeneric : public UK2Node_BaseAttributeActionNode
{
	GENERATED_BODY()
public:
	UK2Node_SetAttributeKeyGeneric();
};

UCLASS()
class UK2Node_SetAttributeKeysGeneric : public UK2Node_BaseAttributeActionNode
{
	GENERATED_BODY()
public:
	UK2Node_SetAttributeKeysGeneric();
};

UCLASS()
class UK2Node_GetAttributeKeyGeneric : public UK2Node_BaseAttributeActionNode
{
	GENERATED_BODY()
public:
	UK2Node_GetAttributeKeyGeneric();
};

UCLASS()
class UK2Node_GetAttributeKeysGeneric : public UK2Node_BaseAttributeActionNode
{
	GENERATED_BODY()
public:
	UK2Node_GetAttributeKeysGeneric();
};

